{
	"FileVersion": 3,
	"Version": 0.1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Rigid Physics Async",
	"Description": "Provides the Chaos Rigid Body Physics Engine (Async Implementation)",
	"Category": "Physics",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosRigidPhysicsAsync",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
	]
}