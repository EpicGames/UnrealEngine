// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
    public class ChaosRigidPhysicsAsync : ModuleRules
    {
        public ChaosRigidPhysicsAsync(ReadOnlyTargetRules Target) : base(Target)
        {
            PublicDependencyModuleNames.AddRange(
                new string[] {
                "Core",
                "CoreUObject",
				"Engine",
                "RigidPhysics",
				}
			);

			SetupModulePhysicsSupport(Target);
			
			//CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
			//StaticAnalyzerDisabledCheckers.Add("cplusplus.NewDeleteLeaks"); // To be reevalulated, believed to be invalid warnings.
			//StaticAnalyzerDisabledCheckers.Add("core.UndefinedBinaryOperatorResult"); // Invalid warning in mass property calculation.
		}
    }
}
