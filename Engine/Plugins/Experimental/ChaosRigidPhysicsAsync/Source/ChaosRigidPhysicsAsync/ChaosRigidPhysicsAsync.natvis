<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="Chaos::Rigids::Async::FShapeInstanceContainerAsyncGT">
    <Intrinsic Name="_IsValidShapeInstance" Expression="Registry._IsValidId(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_GetShapeInstance" Expression="Registry._FindObject(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
  </Type>

  <Type Name="Chaos::Rigids::Async::FShapeInstanceContainerAsyncPT">
    <!--0 <= Index && Index < Registry.Num() && Register[Index].Epoch == Epoch-->
    <Intrinsic Name="_IsValidShapeInstance" Expression="0 &lt;= Id.ArrayIndex &amp;&amp; Id.ArrayIndex &lt; Registry._Num() &amp;&amp; Registry[Id.ArrayIndex].Epoch == Id.Epoch">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_GetShapeInstance" Expression="Registry[Id.ArrayIndex].Object">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
  </Type>

  <Type Name="Chaos::Rigids::Async::FRigidSceneAsyncGT">
    <Intrinsic Name="_IsValidShapeInstance" Expression="ShapeInstanceContainer._IsValidShapeInstance(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_GetShapeInstance" Expression="ShapeInstanceContainer.Registry._FindObject(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>

    <DisplayString>Scene "{*Name.Name,sb}" [GT]</DisplayString>
  </Type>

  <Type Name="Chaos::Rigids::Async::FRigidSceneAsyncPT">
    <Intrinsic Name="_IsValidShapeInstance" Expression="ShapeInstanceContainer._IsValidShapeInstance(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>
    <Intrinsic Name="_GetShapeInstance" Expression="ShapeInstanceContainer._GetShapeInstance(Id)">
      <Parameter Name="Id" Type="UE::Physics::FRigidObjectId&amp;"/>
    </Intrinsic>

    <DisplayString>Scene "{*Name.Name,sb}" [PT]</DisplayString>
  </Type>

  <Type Name="Chaos::Rigids::Async::FRigidBodyPoolAsyncGT">
    <DisplayString>BodyPool{BodyContainerId} "{*Name.Name,sb}" [GT]</DisplayString>
  </Type>

  <Type Name="Chaos::Rigids::Async::FRigidGeometryCollectionAsyncGT">
    <DisplayString>GeometryCollection{BodyContainerId} "{*Name.Name,sb}" [GT]</DisplayString>
  </Type>

  <Type Name="Chaos::Rigids::Async::FRigidBodyAsyncGT">
    <DisplayString Condition="Proxy == nullptr">Body{BodyId} [GT]</DisplayString>
    <DisplayString>Body{BodyId} {*Proxy->Particle} [GT]</DisplayString>
    <Expand>
      <Item Name="Scene">SceneAsync</Item>
      <Item Name="Proxy">Proxy</Item>
      <Item Name="BodyId">BodyId</Item>
      <Synthetic Name="Shapes">
        <DisplayString>Num={ShapeInstanceView.Ids._Num()}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>ShapeInstanceView.Ids._Num()</Size>
            <ValueNode>SceneAsync->_GetShapeInstance(ShapeInstanceView.Ids[$i])</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

  <Type Name="Chaos::Rigids::Async::FRigidBodyAsyncPT">
    <DisplayString Condition="Proxy == nullptr">Body{Handle.BodyId} [PT]</DisplayString>
    <DisplayString>Body{Handle.BodyId} {*Proxy->Handle} [PT]</DisplayString>
    <Expand>
      <Item Name="Scene">Scene</Item>
      <Item Name="Proxy">Proxy</Item>
      <Item Name="Handle">Handle</Item>
      <Synthetic Name="Shapes">
        <DisplayString>Num={ShapeInstanceView.Ids._Num()}</DisplayString>
        <Expand>
          <IndexListItems>
            <Size>ShapeInstanceView.Ids._Num()</Size>
            <ValueNode>Scene->_GetShapeInstance(ShapeInstanceView.Ids[$i])</ValueNode>
          </IndexListItems>
        </Expand>
      </Synthetic>
    </Expand>
  </Type>

</AutoVisualizer>