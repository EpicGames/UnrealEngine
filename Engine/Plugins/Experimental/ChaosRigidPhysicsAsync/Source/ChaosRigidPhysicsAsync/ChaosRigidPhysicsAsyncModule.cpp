// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "RigidPhysics/RigidPhysicsService.h"

class FChaosRigidPhysicsAsyncModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
#if UE_RIGIDPHYSICS_API_ENABLED
		TUniquePtr<Chaos::Rigids::Async::FRigidFactoryAsync> Factory = MakeUnique<Chaos::Rigids::Async::FRigidFactoryAsync>();
		TUniquePtr<Chaos::Rigids::Async::FRigidSceneSettingsAsync> SceneSettings = MakeUnique<Chaos::Rigids::Async::FRigidSceneSettingsAsync>();
		UE::Physics::FRigidPhysicsService& Service = UE::Physics::FRigidPhysicsService::GetInstance();
		Service.RegisterFactory(MoveTemp(Factory), MoveTemp(SceneSettings));
#endif // UE_RIGIDPHYSICS_API_ENABLED
	}

	virtual void ShutdownModule() override
	{
#if UE_RIGIDPHYSICS_API_ENABLED
		UE::Physics::FRigidPhysicsService& Service = UE::Physics::FRigidPhysicsService::GetInstance();
		Service.Reset();
#endif // UE_RIGIDPHYSICS_API_ENABLED
	}
};
IMPLEMENT_MODULE(FChaosRigidPhysicsAsyncModule, ChaosRigidPhysicsAsync)