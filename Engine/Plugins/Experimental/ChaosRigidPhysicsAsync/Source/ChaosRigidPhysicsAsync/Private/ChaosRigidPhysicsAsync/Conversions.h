// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "BodySetupEnums.h"
#include "Chaos/Particles.h"

namespace Chaos::Rigids::Async
{
	inline ECollisionTraceFlag ConvertCollisionTraceFlag(const Chaos::EChaosCollisionTraceFlag InFlag)
	{
		switch (InFlag)
		{
			case Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseDefault:
				return ECollisionTraceFlag::CTF_UseDefault;
			case Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAndComplex:
				return ECollisionTraceFlag::CTF_UseSimpleAndComplex;
			case Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex:
				return ECollisionTraceFlag::CTF_UseSimpleAsComplex;
			case Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple:
				return ECollisionTraceFlag::CTF_UseComplexAsSimple;
			case Chaos::EChaosCollisionTraceFlag::Chaos_CTF_MAX:
				return ECollisionTraceFlag::CTF_MAX;
			default:
			{
				ensure(false);
				return ECollisionTraceFlag::CTF_UseDefault;
			}
		}
	}

	inline Chaos::EChaosCollisionTraceFlag ConvertCollisionTraceFlag(const ECollisionTraceFlag InFlag)
	{
		switch (InFlag)
		{
			case ECollisionTraceFlag::CTF_UseDefault:
				return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseDefault;
			case ECollisionTraceFlag::CTF_UseSimpleAndComplex:
				return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAndComplex;
			case ECollisionTraceFlag::CTF_UseSimpleAsComplex:
				return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseSimpleAsComplex;
			case ECollisionTraceFlag::CTF_UseComplexAsSimple:
				return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseComplexAsSimple;
			case ECollisionTraceFlag::CTF_MAX:
				return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_MAX;
			default:
			{
				ensure(false);
				return Chaos::EChaosCollisionTraceFlag::Chaos_CTF_UseDefault;
			}
		}
	}
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
