// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/JointConstraint6DOFAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"
#include "Chaos/PBDJointConstraintData.h"
#include "Chaos/ChaosConstraintSettings.h"

namespace Chaos::Rigids::Async
{
	Chaos::EJointMotionType Convert(UE::Physics::EJointMotionType InMotionType)
	{
		switch (InMotionType)
		{
			case UE::Physics::EJointMotionType::Free: return Chaos::EJointMotionType::Free;
			case UE::Physics::EJointMotionType::Limited: return Chaos::EJointMotionType::Limited;
			case UE::Physics::EJointMotionType::Locked: return Chaos::EJointMotionType::Locked;
			default: return Chaos::EJointMotionType::Free;
		}
	}

	UE::Physics::EJointMotionType Convert(Chaos::EJointMotionType InMotionType)
	{
		switch (InMotionType)
		{
			case Chaos::EJointMotionType::Free: return UE::Physics::EJointMotionType::Free;
			case Chaos::EJointMotionType::Limited: return UE::Physics::EJointMotionType::Limited;
			case Chaos::EJointMotionType::Locked: return UE::Physics::EJointMotionType::Locked;
			default: return UE::Physics::EJointMotionType::Free;
		}
	}

	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FJointConstraint6DOFAsyncGT, UE::Physics::IJointConstraint6DOF);

	FJointConstraint6DOFAsyncGT::FJointConstraint6DOFAsyncGT(FRigidSceneAsyncGT* InScene)
		: ConstraintId()
		, Scene(InScene)
		, Constraint(nullptr)
	{
		Constraint = new Chaos::FJointConstraint();
	}

	FJointConstraint6DOFAsyncGT::~FJointConstraint6DOFAsyncGT()
	{
		FPhysicsConstraintHandle PhysicsHandle;
		PhysicsHandle.Constraint = Constraint;
		FChaosEngineInterface::ReleaseConstraint(PhysicsHandle);
		Constraint = nullptr;
	}

	FJointConstraintHandle FJointConstraint6DOFAsyncGT::GetHandle() const
	{
		return FJointConstraintHandle(Scene->GetId(), ConstraintId);
	}

	void FJointConstraint6DOFAsyncGT::Activate()
	{
		Scene->ActivateConstraint(this);
	}

	void FJointConstraint6DOFAsyncGT::SetBodies(IRigidBody* Body0, IRigidBody* Body1)
	{
		FRigidBodyAsyncGT* TypedBody0 = Body0->AsA<FRigidBodyAsyncGT>();
		FRigidBodyAsyncGT* TypedBody1 = Body1->AsA<FRigidBodyAsyncGT>();
		// TODO: Handle null (attaching to world anchor)
		if (TypedBody0 && TypedBody1)
		{
			Constraint->SetParticleProxies({ TypedBody0->GetProxy(), TypedBody1->GetProxy() });
		}
	}

	void FJointConstraint6DOFAsyncGT::SetJointTransforms(const FTransform3d& InTransform0, const FTransform3d& InTransform1)
	{
		Constraint->SetJointTransforms({ InTransform0,InTransform1 });
	}

	double FJointConstraint6DOFAsyncGT::GetLinearLimit() const
	{
		return Constraint->GetLinearLimit();
	}

	void FJointConstraint6DOFAsyncGT::SetLinearLimit(const double InLinearLimit)
	{
		Constraint->SetLinearLimit(InLinearLimit);
	}

	UE::Physics::EJointMotionType FJointConstraint6DOFAsyncGT::GetLinearMotionTypesX() const
	{
		return Convert(Constraint->GetLinearMotionTypesX());
	}

	void FJointConstraint6DOFAsyncGT::SetLinearMotionTypesX(const EJointMotionType InMotionType)
	{
		Constraint->SetLinearMotionTypesX(Convert(InMotionType));
	}

	UE::Physics::EJointMotionType FJointConstraint6DOFAsyncGT::GetLinearMotionTypesY() const
	{
		return Convert(Constraint->GetLinearMotionTypesY());
	}

	void FJointConstraint6DOFAsyncGT::SetLinearMotionTypesY(const EJointMotionType InMotionType)
	{
		Constraint->SetLinearMotionTypesY(Convert(InMotionType));
	}

	UE::Physics::EJointMotionType FJointConstraint6DOFAsyncGT::GetLinearMotionTypesZ() const
	{
		return Convert(Constraint->GetLinearMotionTypesZ());
	}

	void FJointConstraint6DOFAsyncGT::SetLinearMotionTypesZ(const EJointMotionType InMotionType)
	{
		Constraint->SetLinearMotionTypesZ(Convert(InMotionType));
	}

	void FJointConstraint6DOFAsyncGT::SetAngularMotionTypesX(const EJointMotionType InMotionType)
	{
		Constraint->SetAngularMotionTypesX(Convert(InMotionType));
	}

	void FJointConstraint6DOFAsyncGT::SetAngularMotionTypesY(const EJointMotionType InMotionType)
	{
		Constraint->SetAngularMotionTypesY(Convert(InMotionType));
	}

	void FJointConstraint6DOFAsyncGT::SetAngularMotionTypesZ(const EJointMotionType InMotionType)
	{
		Constraint->SetAngularMotionTypesZ(Convert(InMotionType));
	}

	void FJointConstraint6DOFAsyncGT::SetAngularLimits(const FVector3d& InAngularLimits)
	{
		Constraint->SetAngularLimits(InAngularLimits);
	}

	void FJointConstraint6DOFAsyncGT::SetCollisionEnabled(const bool bCollisionEnabled)
	{
		Constraint->SetCollisionEnabled(bCollisionEnabled);
	}

	void FJointConstraint6DOFAsyncGT::SetProjectionEnabled(const bool bProjectionEnabled)
	{
		Constraint->SetProjectionEnabled(bProjectionEnabled);
	}

	void FJointConstraint6DOFAsyncGT::SetMassConditioningEnabled(const bool bMassConditioningEnabled)
	{
		Constraint->SetMassConditioningEnabled(bMassConditioningEnabled);
	}

	void FJointConstraint6DOFAsyncGT::SetUseLinearSolver(const bool bUseLinearSolver)
	{
		Constraint->SetUseLinearSolver(bUseLinearSolver);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftLinearLimitsEnabled(const bool bSoftLinearLimitsEnabled)
	{
		Constraint->SetSoftLinearLimitsEnabled(bSoftLinearLimitsEnabled);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftTwistLimitsEnabled(const bool bSoftTwistLimitsEnabled)
	{
		Constraint->SetSoftTwistLimitsEnabled(bSoftTwistLimitsEnabled);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftSwingLimitsEnabled(const bool bSoftSwingLimitsEnabled)
	{
		Constraint->SetSoftSwingLimitsEnabled(bSoftSwingLimitsEnabled);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftLinearStiffness(const double InSoftLinearStiffness)
	{
		Constraint->SetSoftLinearStiffness(InSoftLinearStiffness);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftLinearDamping(const double InSoftLinearDamping)
	{
		Constraint->SetSoftLinearDamping(InSoftLinearDamping);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftTwistStiffness(const double InSoftTwistStiffness)
	{
		Constraint->SetSoftTwistStiffness(InSoftTwistStiffness);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftTwistDamping(const double InSoftTwistDamping)
	{
		Constraint->SetSoftTwistDamping(InSoftTwistDamping);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftSwingStiffness(const double InSoftSwingStiffness)
	{
		Constraint->SetSoftSwingStiffness(InSoftSwingStiffness);
	}

	void FJointConstraint6DOFAsyncGT::SetSoftSwingDamping(const double InSoftSwingDamping)
	{
		Constraint->SetSoftSwingDamping(InSoftSwingDamping);
	}

	void FJointConstraint6DOFAsyncGT::SetProjectionLinearAlpha(const double InProjectionLinearAlpha)
	{
		Constraint->SetProjectionLinearAlpha(InProjectionLinearAlpha);
	}

	void FJointConstraint6DOFAsyncGT::SetProjectionAngularAlpha(const double InProjectionAngularAlpha)
	{
		Constraint->SetProjectionAngularAlpha(InProjectionAngularAlpha);
	}

	void FJointConstraint6DOFAsyncGT::SetProjectionLinearTolerance(const double InProjectionLinearTolerance)
	{
		Constraint->SetProjectionLinearTolerance(InProjectionLinearTolerance);
	}

	void FJointConstraint6DOFAsyncGT::SetProjectionAngularTolerance(const double InProjectionAngularTolerance)
	{
		Constraint->SetProjectionAngularTolerance(InProjectionAngularTolerance);
	}

	void FJointConstraint6DOFAsyncGT::SetLinearContactDistance(const double InLinearContactDistance)
	{
		Constraint->SetLinearContactDistance(InLinearContactDistance);
	}

	void FJointConstraint6DOFAsyncGT::SetTwistContactDistance(const double InTwistContactDistance)
	{
		Constraint->SetTwistContactDistance(InTwistContactDistance);
	}

	void FJointConstraint6DOFAsyncGT::SetSwingContactDistance(const double InSwingContactDistance)
	{
		Constraint->SetSwingContactDistance(InSwingContactDistance);
	}

	void FJointConstraint6DOFAsyncGT::SetAngularDriveStiffness(const FVector3d& InAngularDriveStiffness)
	{
		Constraint->SetAngularDriveStiffness(InAngularDriveStiffness);
	}

	void FJointConstraint6DOFAsyncGT::SetAngularDriveDamping(const FVector3d& InAngularDriveDamping)
	{
		Constraint->SetAngularDriveDamping(InAngularDriveDamping);
	}

	void FJointConstraint6DOFAsyncGT::SetLinearDriveStiffness(const FVector3d& InLinearDriveStiffness)
	{
		Constraint->SetLinearDriveStiffness(InLinearDriveStiffness);
	}

	void FJointConstraint6DOFAsyncGT::SetLinearDriveDamping(const FVector3d& InLinearDriveDamping)
	{
		Constraint->SetLinearDriveDamping(InLinearDriveDamping);
	}


	void FJointConstraint6DOFAsyncGT::SetId(const FRigidObjectId& InId)
	{
		ConstraintId = InId;
	}

	Chaos::FJointConstraint* FJointConstraint6DOFAsyncGT::GetJointConstraint()
	{
		return Constraint;
	}

	const Chaos::FJointConstraint* FJointConstraint6DOFAsyncGT::GetJointConstraint() const
	{
		return Constraint;
	}

	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FJointConstraint6DOFAsyncPT, UE::Physics::IJointConstraint6DOF);

	FJointConstraint6DOFAsyncPT::FJointConstraint6DOFAsyncPT(const FJointConstraintHandle& InHandle, FRigidSceneAsyncPT* InScene, Chaos::FJointConstraint* InConstraint)
		: Handle(InHandle)
		, Scene(InScene)
		, Constraint(InConstraint)
	{
	}

	FJointConstraintHandle FJointConstraint6DOFAsyncPT::GetHandle() const
	{
		return Handle;
	}

	double FJointConstraint6DOFAsyncPT::GetLinearLimit() const
	{
		return Constraint->GetLinearLimit();
	}

	void FJointConstraint6DOFAsyncPT::SetLinearLimit(const double InLinearLimit)
	{
		Constraint->SetLinearLimit(InLinearLimit);
	}

	UE::Physics::EJointMotionType FJointConstraint6DOFAsyncPT::GetLinearMotionTypesX() const
	{
		return Convert(Constraint->GetLinearMotionTypesX());
	}

	void FJointConstraint6DOFAsyncPT::SetLinearMotionTypesX(const EJointMotionType InMotionType)
	{
		Constraint->SetLinearMotionTypesX(Convert(InMotionType));
	}

	UE::Physics::EJointMotionType FJointConstraint6DOFAsyncPT::GetLinearMotionTypesY() const
	{
		return Convert(Constraint->GetLinearMotionTypesY());
	}

	void FJointConstraint6DOFAsyncPT::SetLinearMotionTypesY(const EJointMotionType InMotionType)
	{
		Constraint->SetLinearMotionTypesY(Convert(InMotionType));
	}

	UE::Physics::EJointMotionType FJointConstraint6DOFAsyncPT::GetLinearMotionTypesZ() const
	{
		return Convert(Constraint->GetLinearMotionTypesZ());
	}

	void FJointConstraint6DOFAsyncPT::SetLinearMotionTypesZ(const EJointMotionType InMotionType)
	{
		Constraint->SetLinearMotionTypesZ(Convert(InMotionType));
	}
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
