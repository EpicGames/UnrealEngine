// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/RigidBodyAsync.h"

#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"
#include "Chaos/ShapeInstance.h"
#include "RigidPhysics/RigidBody.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidBodyAsyncGT, UE::Physics::IRigidBody);

	double SafeInvert(double InValue, double InDefaultInverse = 0.0)
	{
		if (InValue != 0.0)
		{
			return 1.0 / InValue;
		}
		return InDefaultInverse;
	}

	FVector3d SafeInvert(const FVector3d& InValue, double InDefaultInverse = 0.0)
	{
		return FVector3d(
			SafeInvert(InValue.X, InDefaultInverse),
			SafeInvert(InValue.Y, InDefaultInverse),
			SafeInvert(InValue.Z, InDefaultInverse));
	}

	FRigidBodyAsyncGT::FRigidBodyAsyncGT(FRigidSceneAsyncGT* InScene, const UE::Physics::FRigidDebugName& InName, UE::Physics::ERigidMovementType InMovementType)
		: BodyId()
		, SceneAsync(InScene)
		, Proxy(nullptr)
	{
		using namespace UE::Physics;
		UE_RIGIDPHYSICS_CHECK(SceneAsync != nullptr);

		TUniquePtr<FGeometryParticle> Particle = nullptr;
		if (InMovementType == ERigidMovementType::Static)
		{
			Particle = FGeometryParticle::CreateParticle();
		}
		else
		{
			TUniquePtr<FPBDRigidParticle> RigidParticle = FPBDRigidParticle::CreateParticle();
			if (InMovementType == ERigidMovementType::Kinematic)
			{
				RigidParticle->SetObjectState(EObjectStateType::Kinematic);
			}
			Particle = MoveTemp(RigidParticle);
		}

		UE_RIGIDPHYSICS_CHECK(Particle != nullptr);
		if (Particle != nullptr)
		{
#if CHAOS_DEBUG_NAME
			Particle->SetDebugName(InName.SharedValue());
#endif
			Proxy = FSingleParticlePhysicsProxy::Create(MoveTemp(Particle));
		}
	}

	FRigidBodyAsyncGT::~FRigidBodyAsyncGT()
	{
		UE_RIGIDPHYSICS_CHECK(!IsActive());

		// If we were never deactivated, our proxy will not have been destroyed. See Deactivate().
		if (Proxy != nullptr)
		{
			delete Proxy;
		}
	}

	UE::Physics::FRigidBodyHandle FRigidBodyAsyncGT::GetHandle() const
	{
		using namespace UE::Physics;

		return FRigidBodyHandle(GetScene()->GetId(), BodyId);
	}

	void FRigidBodyAsyncGT::SetId(const UE::Physics::FRigidObjectId& InId)
	{
		BodyId = InId;
	}

	bool FRigidBodyAsyncGT::IsValid() const
	{
		return (Proxy != nullptr) && (Proxy->GetParticle_LowLevel() != nullptr);
	}

	const FSingleParticlePhysicsProxy* FRigidBodyAsyncGT::GetProxy() const
	{
		return Proxy;
	}

	FSingleParticlePhysicsProxy* FRigidBodyAsyncGT::GetProxy()
	{
		return Proxy;
	}

	const FRigidBodyHandle_External& FRigidBodyAsyncGT::GetProxyAPI() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		return Proxy->GetGameThreadAPI();
	}

	FRigidBodyHandle_External& FRigidBodyAsyncGT::GetProxyAPI()
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		return Proxy->GetGameThreadAPI();
	}

	FRigidSceneAsyncGT* FRigidBodyAsyncGT::GetScene() const
	{
		return SceneAsync;
	}

	bool FRigidBodyAsyncGT::IsActive() const
	{
		// TODO_CHAOSAPI: need a better way to know if we're active in the scene
		// This works for now because UniqueIdx is set in FPBDRigidsSolver::RegisterObject 
		// and GetMarkDeleted() is true after FPBDRigidsSolver::UnregisterObject.
		return (Proxy != nullptr) && !Proxy->GetMarkedDeleted() && (Proxy->GetParticle_LowLevel() != nullptr) && Proxy->GetParticle_LowLevel()->UniqueIdx().IsValid();
	}

	void FRigidBodyAsyncGT::Activate()
	{
		// Cannot re-activate after deactivate because the proxy is destroyed by UnregisterObject
		// TODO_CHAOSAPI: Fix this - Proxy lifetime should be shared and PT should only control
		// lifetime of PT elements.
		if (Proxy != nullptr)
		{
			if (FRigidSceneAsyncGT* Scene = GetScene())
			{
				Scene->ActivateBody(this);
			}
		}
	}

	void FRigidBodyAsyncGT::Deactivate()
	{
		if (FRigidSceneAsyncGT* Scene = GetScene())
		{
			Scene->DeactivateBody(this);

			// Cannot re-activate after deactivate because the proxy is destroyed by UnregisterObject.
			Proxy = nullptr;
		}
	}

	bool FRigidBodyAsyncGT::IsStatic() const
	{
		return (GetProxyAPI().ObjectState() == EObjectStateType::Static);
	}

	bool FRigidBodyAsyncGT::IsKinematic() const
	{
		return (GetProxyAPI().ObjectState() == EObjectStateType::Kinematic);
	}

	bool FRigidBodyAsyncGT::IsDynamic() const
	{
		return (GetProxyAPI().ObjectState() == EObjectStateType::Dynamic) || (GetProxyAPI().ObjectState() == EObjectStateType::Sleeping);
	}

	bool FRigidBodyAsyncGT::IsSleeping() const
	{
		return (GetProxyAPI().ObjectState() == EObjectStateType::Sleeping);
	}

	int32 FRigidBodyAsyncGT::GetNumShapes() const
	{
		return ShapeInstanceView.Num();
	}

	FRigidShapeInstanceAsync* FRigidBodyAsyncGT::GetShape(int32 InShapeIndex) const
	{
		return SceneAsync->GetShape(ShapeInstanceView, InShapeIndex);
	}

	FRigidShapeInstanceAsync* FRigidBodyAsyncGT::CreateShape(const UE::Physics::FRigidShapeInstanceSetup& InSetup)
	{
		return CreateShapeInternal(FRigidShapeInstanceAsync::Create(InSetup));
	}

	FRigidShapeInstanceAsync* FRigidBodyAsyncGT::CreateShape(TUniquePtr<Chaos::FPerShapeData>&& InShapeData)
	{
		return CreateShapeInternal(InShapeData.Release());
	}

	void FRigidBodyAsyncGT::DestroyShape(IRigidShapeInstance* InShape, bool bWakeTouching)
	{
		if (FRigidShapeInstanceAsync* CastedShape = InShape->AsAChecked<FRigidShapeInstanceAsync>())
		{
			// Remove the shape from our view
			SceneAsync->RemoveShape(*this, *CastedShape);

			// Currently, the proxy owns the shape, so also remove the memory
			GetProxyAPI().RemoveShape(CastedShape->PerShapeData, bWakeTouching);
			CastedShape->PerShapeData = nullptr;

			SceneAsync->DestroyShape(CastedShape);
		}
	}

	FShapeInstanceView& FRigidBodyAsyncGT::GetShapeInstanceView()
	{
		return ShapeInstanceView;
	}

	void FRigidBodyAsyncGT::InitTransform(const FTransform& InTransform)
	{
		// TODO_CHAOSAPI: This is not supported by the Proxy on the GT. 
		// For now equivalent to UpdateTransform.
		UpdateTransform(InTransform);
	}

	void FRigidBodyAsyncGT::UpdateTransform(const FTransform& InTransform)
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());

		FRigidBodyHandle_External& ProxyAPI = GetProxyAPI();
		if (!IsKinematic() && !IsSleeping() && FVec3::IsNearlyEqual(InTransform.GetLocation(), ProxyAPI.X(), SMALL_NUMBER) && FRotation3::IsNearlyEqual(InTransform.GetRotation(), ProxyAPI.R(), SMALL_NUMBER))
		{
			// Don't update X/R if they haven't changed. This allows scale to be set on simulating body without overriding async position/rotation.
			return;
		}

		if (IsKinematic())
		{
			// NOTE: UpdateTransform is a teleport for kinematics. Use SetKinematicTarget
			// if the kinematic should calculate its velocity from the transform delta.
			ProxyAPI.SetKinematicTarget(InTransform);
			ProxyAPI.SetV(FVector::Zero());
			ProxyAPI.SetW(FVector::Zero());
		}

		ProxyAPI.SetX(InTransform.GetTranslation());
		ProxyAPI.SetR(InTransform.GetRotation());
		ProxyAPI.UpdateShapeBounds();

		if (FRigidSceneAsyncGT* Scene = GetScene())
		{
			Scene->OnRigidBodyTransformUpdated(this);
		}
	}

	void FRigidBodyAsyncGT::SetKinematicTarget(const FTransform& InTransform)
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		GetProxyAPI().SetKinematicTarget(FKinematicTarget::MakePositionTarget(InTransform.GetTranslation(), InTransform.GetRotation()));
	}

	FTransform FRigidBodyAsyncGT::GetTransform() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		return FTransform(GetProxyAPI().R(), GetProxyAPI().X());
	}

	FBounds3d FRigidBodyAsyncGT::GetBounds() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		FConstGenericParticleHandle ParticleHandle = Proxy->GetHandle_LowLevel();
		FAABB3 Bounds;
		if (GetProxyAPI().GetGeometry()->HasBoundingBox())
		{
			Bounds = GetProxyAPI().GetGeometry()->CalculateTransformedBounds(GetTransform());
		}
		else
		{
			Bounds = FAABB3::FullAABB();
		}
		return FBounds3d(Bounds.Min(), Bounds.Max());
	}

	double FRigidBodyAsyncGT::GetMass() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		return GetProxyAPI().M();
	}

	void FRigidBodyAsyncGT::SetMass(double InMass)
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		GetProxyAPI().SetM(InMass);
		GetProxyAPI().SetInvM(SafeInvert(InMass));
	}

	FVector3d FRigidBodyAsyncGT::GetInertia() const
	{
		return FVector3d(GetProxyAPI().I());
	}

	void FRigidBodyAsyncGT::SetInertia(const FVector3d& InInertia)
	{
		GetProxyAPI().SetI(FVec3f(InInertia));
		GetProxyAPI().SetInvI(FVec3f(SafeInvert(InInertia)));
	}

	//void SetCCDEnabled(bool bInEnabled) {  }
	//void SetMACDEnabled(bool bInEnabled) {  }
	//void SetSmoothEdgeCollisionsEnabled(bool bInEnabled) {  }
	//void SetMaxLinearVelocity(float InMax) {  }
	//void SetMaxAngularVelocity(float InMax) {  }
	//void SetInertiaConditioningEnabled(bool bInEnabled) {  }
	//void SetGyroscopicTorqueEnabled(bool bInEnabled) {  }
	//void SetGravityGroup(int32 InId) {  }

	void FRigidBodyAsyncGT::SetCenterOfMass(const FVector3d& InCenterOfMass)
	{
		GetProxyAPI().SetCenterOfMass(InCenterOfMass);
	}

	void FRigidBodyAsyncGT::SetRotationOfMass(const FQuat& InRotationOfMass)
	{
		GetProxyAPI().SetRotationOfMass(InRotationOfMass);
	}

	void FRigidBodyAsyncGT::SetGravityScale(float InScale)
	{ 
		UE_RIGIDPHYSICS_CHECK(IsValid());
		GetProxyAPI().SetGravityEnabled(InScale > 0.0f);
	}

	//void SetWakeEventsEnabled(bool bInEnabled) {  }
	//void SetNumPositionIterations(int32 InNum) {  }
	//void SetNumVelocityIterations(int32 InNum) {  }
	//void SetNumProjectionIterations(int32 InNum) {  }
	void FRigidBodyAsyncGT::SetIsSleeping(const bool bInSleeping)
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());

		Chaos::FRigidBodyHandle_External& BodyHandle_External = GetProxyAPI();
		const Chaos::EObjectStateType ObjectState = BodyHandle_External.ObjectState();
		// NOTE: We want to set the state whether or not the current state matches - if the physics
		// thread has queued up a conflicting wake/sleep event, this manual call should take priority.
		if (ObjectState == Chaos::EObjectStateType::Dynamic || ObjectState == Chaos::EObjectStateType::Sleeping)
		{
			if (bInSleeping)
			{
				BodyHandle_External.SetObjectState(Chaos::EObjectStateType::Sleeping);
			}
			else
			{
				BodyHandle_External.SetObjectState(Chaos::EObjectStateType::Dynamic);
				BodyHandle_External.ClearEvents();
			}
		}
	}
	
	FRigidShapeInstanceAsync* FRigidBodyAsyncGT::CreateShapeInternal(Chaos::FPerShapeData* InShapeData)
	{
		using namespace Chaos;
		TArray<FImplicitObjectPtr> ImplicitsObjects;
		ImplicitsObjects = { InShapeData->GetGeometry() };
		GetProxyAPI().MergeGeometry(MoveTemp(ImplicitsObjects));

		// TODO_CHAOSAPI: We are wrapping the instance's PerShapeData in a UniquePtr here
		// which is leaving a raw pointer to the same object behind.
		// This is temporary until the API is used everywhere and we can refactor.
		FShapesArray Shapes;
		Shapes.Add(TUniquePtr<FPerShapeData>(InShapeData));
		GetProxyAPI().MergeShapesArray(MoveTemp(Shapes));

		FRigidShapeInstanceAsync* ShapeInstance = SceneAsync->AllocateShape();
		ShapeInstance->PerShapeData = InShapeData;
		SceneAsync->AddShape(*this, *ShapeInstance);
		return ShapeInstance;
	}
} // namespace Chaos::Rigids::Async

namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidBodyAsyncPT, UE::Physics::IRigidBody);

	FRigidBodyAsyncPT::FRigidBodyAsyncPT(
		FRigidSceneAsyncPT* InScene,
		const UE::Physics::FRigidBodyHandle& InBodyHandle,
		FSingleParticlePhysicsProxy* InProxy)
		: Scene(InScene)
		, Handle(InBodyHandle)
		, Proxy(InProxy)
	{
	}

	FRigidBodyAsyncPT::~FRigidBodyAsyncPT()
	{
	}

	UE::Physics::FRigidBodyHandle FRigidBodyAsyncPT::GetHandle() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		return Handle;
	}

	bool FRigidBodyAsyncPT::IsValid() const
	{
		return (Proxy != nullptr) && (Proxy->GetHandle_LowLevel() != nullptr);
	}

	bool FRigidBodyAsyncPT::IsStatic() const
	{
		return (GetProxyAPI()->ObjectState() == EObjectStateType::Static);
	}

	bool FRigidBodyAsyncPT::IsKinematic() const
	{
		return (GetProxyAPI()->ObjectState() == EObjectStateType::Kinematic);
	}

	bool FRigidBodyAsyncPT::IsDynamic() const
	{
		return (GetProxyAPI()->ObjectState() == EObjectStateType::Dynamic) || (GetProxyAPI()->ObjectState() == EObjectStateType::Sleeping);
	}

	bool FRigidBodyAsyncPT::IsSleeping() const
	{
		return (GetProxyAPI()->ObjectState() == EObjectStateType::Sleeping);
	}

	int32 FRigidBodyAsyncPT::GetNumShapes() const
	{
		return ShapeInstanceView.Num();
	}

	FRigidShapeInstanceAsync* FRigidBodyAsyncPT::GetShape(int32 InShapeIndex) const
	{
		return Scene->GetShape(ShapeInstanceView, InShapeIndex);
	}

	FShapeInstanceView& FRigidBodyAsyncPT::GetShapeInstanceView()
	{
		return ShapeInstanceView;
	}

	void FRigidBodyAsyncPT::UpdateShapeInstance(FRigidShapeInstanceAsync& ShapeInstance, const int32 InShapeIndex)
	{
		const Chaos::FShapesArray& ShapesArray = GetProxyAPI()->ShapesArray();
		// Note: This index isn't always valid so we have to guard. This is true if multiple RemoveShape calls happen in one tick.
		if (ShapesArray.IsValidIndex(InShapeIndex))
		{
			ShapeInstance.PerShapeData = ShapesArray[InShapeIndex].Get();
		}
		else
		{
			ShapeInstance.PerShapeData = nullptr;
		}
	}

	void FRigidBodyAsyncPT::InitTransform(const FTransform& InTransform)
	{
		SetTransform(InTransform,  /*bIsTeleport*/true);
	}

	void FRigidBodyAsyncPT::UpdateTransform(const FTransform& InTransform)
	{
		SetTransform(InTransform,  /*bIsTeleport*/false);
	}

	void FRigidBodyAsyncPT::SetTransform(const FTransform& InTransform, bool bIsTeleport)
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		if (!IsDynamic())
		{
			// TODO_CHAOSAPI: Statics are assumed to be controlled only by the GT. We should track changes that need 
			// to be sent back to the GT (this is also true for various properties like EtherDrag, and not just on Statics).
			UE_LOGFMT(
				LogRigidPhysics, Error, 
				"InitTransform and UpdateTransform on Static or Kinematic Body '{0}' not supported on the Physics Thread. Transform would diverge from Game Thread.", 
				*GetDebugName()
				);
			return;
		}

		if (FPBDRigidsSolver* Solver = Proxy->GetSolver<FPBDRigidsSolver>())
		{
			// NOTE: The Chaos Proxy API does not do everything required when changing
			// properties on the particle. E.g., marking dirty, etc.
			// Here'what should be in GetProxyAPI()->SetX()/SetR()
			FGeometryParticleHandle* ParticleHandle = Proxy->GetHandle_LowLevel();
			Solver->GetEvolution()->SetParticleTransform(ParticleHandle, InTransform.GetTranslation(), InTransform.GetRotation(), bIsTeleport);
			Solver->GetEvolution()->DirtyParticle(*ParticleHandle);
		}
	}

	void FRigidBodyAsyncPT::SetKinematicTarget(const FTransform& InTransform)
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		if (FPBDRigidsSolver* Solver = Proxy->GetSolver<FPBDRigidsSolver>())
		{
			FGeometryParticleHandle* ParticleHandle = Proxy->GetHandle_LowLevel();
			Solver->GetEvolution()->SetParticleKinematicTarget(ParticleHandle, FKinematicTarget::MakePositionTarget(InTransform.GetTranslation(), InTransform.GetRotation()));
			Solver->GetEvolution()->DirtyParticle(*ParticleHandle);
		}
	}

	FTransform FRigidBodyAsyncPT::GetTransform() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		FConstGenericParticleHandle ParticleHandle = Proxy->GetHandle_LowLevel();
		return FTransform(ParticleHandle->Q(), ParticleHandle->P());
	}

	FBounds3d FRigidBodyAsyncPT::GetBounds() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		FConstGenericParticleHandle ParticleHandle = Proxy->GetHandle_LowLevel();
		return FBounds3d(ParticleHandle->WorldSpaceInflatedBounds().Min(), ParticleHandle->WorldSpaceInflatedBounds().Max());
	}

	double FRigidBodyAsyncPT::GetMass() const
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		FConstGenericParticleHandle ParticleHandle = Proxy->GetHandle_LowLevel();
		return ParticleHandle->M();
	}

	void FRigidBodyAsyncPT::SetMass(double InMass)
	{
		UE_RIGIDPHYSICS_CHECK(IsValid());
		ensure(false);
		// TODO_CHAOSAPI: Not yet supported on the PT because the mass will not make it back to GT.
		//FGenericParticleHandle ParticleHandle = Proxy->GetHandle_LowLevel();
		//return ParticleHandle->SetM(InMass);
	}
	
	const FString FRigidBodyAsyncPT::GetDebugName() const
	{
		static FString UnknownName(TEXT("Unknown"));
#if CHAOS_DEBUG_NAME
		const FRigidBodyHandle_Internal* ProxyAPI = GetProxyAPI();
		return ProxyAPI->DebugName().IsValid() ? *ProxyAPI->DebugName() : UnknownName;
#else
		return UnknownName;
#endif
	};
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
