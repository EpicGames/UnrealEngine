// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"

#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "Logging/StructuredLog.h"
#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidLog.h"
#include "RigidPhysics/RigidSceneRegistry.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::Rigids::Async
{
	FRigidFactoryAsync::FRigidFactoryAsync()
	{
	}

	FRigidFactoryAsync::~FRigidFactoryAsync()
	{
		// Make sure all scenes were destroyed
		check(Scenes.IsEmpty());
	}

	const UE::Physics::FRigidTypeId& FRigidFactoryAsync::GetSceneTypeId() const
	{
		return FRigidSceneAsyncGT::GetStaticTypeId();
	}

	UE::Physics::FRigidSceneHandle FRigidFactoryAsync::CreateScene(const UE::Physics::FRigidDebugName& InName, const UE::Physics::IRigidSceneSettings* InSettings)
	{
		using namespace UE::Physics;
		check(InSettings != nullptr);

		const FRigidSceneSettingsAsync* SettingsAsync = InSettings->AsA<FRigidSceneSettingsAsync>();

		UE_CLOGFMT((SettingsAsync == nullptr), LogRigidPhysics, Error,
			"FRigidFactoryAsync::CreateScene: Cannot create scene. Expected settings of type '{0}' but received '{1}'.",
			FRigidSceneSettingsAsync::GetStaticTypeId().GetTypeName().GetCharArray().GetData(),
			InSettings->GetTypeId().GetTypeName().GetCharArray().GetData());

		// Create a scene and store a hard reference to prevent its destruction
		if (SettingsAsync != nullptr)
		{
			if (FRigidSceneAsyncGT* Scene = new FRigidSceneAsyncGT(InName, *SettingsAsync))
			{
				Scene->SetId(FRigidSceneRegistry::GetInstance().Register(Scene));
				Scenes.Add(Scene);

				Scene->Startup();

				return FRigidSceneHandle(Scene);
			}
		}

		return FRigidSceneHandle();
	}

	void FRigidFactoryAsync::DestroyScene(const UE::Physics::FRigidSceneHandle& InSceneHandle)
	{
		using namespace UE::Physics;

		if (IRigidScene* Scene = InSceneHandle.GetScene())
		{
			if (FRigidSceneAsyncGT* SceneGT = Scene->AsA<FRigidSceneAsyncGT>())
			{
				check(Scenes.Contains(SceneGT));
			
				SceneGT->Shutdown();

				Scenes.Remove(SceneGT);
				FRigidSceneRegistry::GetInstance().Unregister(SceneGT->GetId());

				delete SceneGT;
			}
		}
	}
}

#endif // UE_RIGIDPHYSICS_API_ENABLED