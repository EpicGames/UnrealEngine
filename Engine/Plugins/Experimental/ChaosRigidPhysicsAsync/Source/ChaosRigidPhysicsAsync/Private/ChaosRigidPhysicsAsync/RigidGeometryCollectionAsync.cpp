// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/RigidGeometryCollectionAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidGeometryCollectionBodyAsyncGT, UE::Physics::IRigidBody);


	FRigidGeometryCollectionBodyAsyncGT::FRigidGeometryCollectionBodyAsyncGT(FRigidGeometryCollectionAsyncGT* InGeometryCollection)
	{
	}

	FRigidGeometryCollectionBodyAsyncGT::~FRigidGeometryCollectionBodyAsyncGT()
	{
	}

	UE::Physics::FRigidBodyHandle FRigidGeometryCollectionBodyAsyncGT::GetHandle() const
	{
		using namespace UE::Physics;

		return FRigidBodyHandle();
	}

	bool FRigidGeometryCollectionBodyAsyncGT::IsActive() const
	{
		return false;
	}

	void FRigidGeometryCollectionBodyAsyncGT::Activate()
	{
	}

	void FRigidGeometryCollectionBodyAsyncGT::Deactivate()
	{
	}

	bool FRigidGeometryCollectionBodyAsyncGT::IsStatic() const
	{
		return false;
	}

	bool FRigidGeometryCollectionBodyAsyncGT::IsKinematic() const
	{
		return false;
	}

	bool FRigidGeometryCollectionBodyAsyncGT::IsDynamic() const
	{
		return false;
	}

	bool FRigidGeometryCollectionBodyAsyncGT::IsSleeping() const
	{
		return false;
	}

	int32 FRigidGeometryCollectionBodyAsyncGT::GetNumShapes() const
	{
		return 0;
	}

	UE::Physics::IRigidShapeInstance* FRigidGeometryCollectionBodyAsyncGT::GetShape(int32 InShapeIndex) const
	{
		return nullptr;
	}
}


namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidGeometryCollectionBodyAsyncPT, UE::Physics::IRigidBody);


	FRigidGeometryCollectionBodyAsyncPT::FRigidGeometryCollectionBodyAsyncPT(FRigidGeometryCollectionAsyncPT* InGeometryCollection)
	{
	}

	FRigidGeometryCollectionBodyAsyncPT::~FRigidGeometryCollectionBodyAsyncPT()
	{
	}

	UE::Physics::FRigidBodyHandle FRigidGeometryCollectionBodyAsyncPT::GetHandle() const
	{
		using namespace UE::Physics;

		return FRigidBodyHandle();
	}

	bool FRigidGeometryCollectionBodyAsyncPT::IsActive() const
	{
		return false;
	}

	void FRigidGeometryCollectionBodyAsyncPT::Activate()
	{
	}

	void FRigidGeometryCollectionBodyAsyncPT::Deactivate()
	{
	}

	bool FRigidGeometryCollectionBodyAsyncPT::IsStatic() const
	{
		return false;
	}

	bool FRigidGeometryCollectionBodyAsyncPT::IsKinematic() const
	{
		return false;
	}

	bool FRigidGeometryCollectionBodyAsyncPT::IsDynamic() const
	{
		return false;
	}

	bool FRigidGeometryCollectionBodyAsyncPT::IsSleeping() const
	{
		return false;
	}

	int32 FRigidGeometryCollectionBodyAsyncPT::GetNumShapes() const
	{
		return 0;
	}

	UE::Physics::IRigidShapeInstance* FRigidGeometryCollectionBodyAsyncPT::GetShape(int32 InShapeIndex) const
	{
		return nullptr;
	}
}


namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidGeometryCollectionAsyncGT, UE::Physics::IRigidGeometryCollection);


	FRigidGeometryCollectionAsyncGT::FRigidGeometryCollectionAsyncGT(FRigidSceneAsyncGT* InScene, const UE::Physics::FRigidDebugName& InName, const FSimulationParameters& InParameters)
		: Name(InName)
		, BodyContainerId()
		, Scene(InScene)
		, Proxy(nullptr)
	{
		UE_RIGIDPHYSICS_CHECK(Scene != nullptr);

	}

	FRigidGeometryCollectionAsyncGT::~FRigidGeometryCollectionAsyncGT()
	{
	}

	UE::Physics::FRigidBodyContainerHandle FRigidGeometryCollectionAsyncGT::GetHandle() const
	{
		using namespace UE::Physics;

		return FRigidBodyContainerHandle(Scene->GetId(), BodyContainerId);
	}

	int32 FRigidGeometryCollectionAsyncGT::GetNumBodies() const
	{
		if (Proxy != nullptr)
		{
			return Proxy->GetNumTransforms();
		}
		return 0;
	}

	FRigidGeometryCollectionBodyAsyncGT* FRigidGeometryCollectionAsyncGT::GetBodyAt(int32 BodyIndex) const
	{
		if (Proxy != nullptr)
		{
		}
		return nullptr;
	}

	FRigidGeometryCollectionBodyAsyncGT* FRigidGeometryCollectionAsyncGT::GetBody(const UE::Physics::FRigidObjectId& InBodyId) const
	{
		return nullptr;
	}

	void FRigidGeometryCollectionAsyncGT::SetId(const UE::Physics::FRigidObjectId& InBodyContainerId)
	{
		BodyContainerId = InBodyContainerId;
	}

}

namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidGeometryCollectionAsyncPT, UE::Physics::IRigidGeometryCollection);

	FRigidGeometryCollectionAsyncPT::FRigidGeometryCollectionAsyncPT(FRigidSceneAsyncPT* InScene, const UE::Physics::FRigidObjectId& InId, FGeometryCollectionPhysicsProxy* InProxy)
	 : BodyContainerId(InId)
	 , Scene(InScene)
	 , Proxy(InProxy)
	{

	}

	FRigidGeometryCollectionAsyncPT::~FRigidGeometryCollectionAsyncPT()
	{

	}

	UE::Physics::FRigidBodyContainerHandle FRigidGeometryCollectionAsyncPT::GetHandle() const
	{
		using namespace UE::Physics;

		return FRigidBodyContainerHandle(Scene->GetId(), BodyContainerId);
	}

	void FRigidGeometryCollectionAsyncPT::SetId(const UE::Physics::FRigidObjectId& InBodyContainerId)
	{
		// Should not be called on the PT
		UE_RIGIDPHYSICS_CHECK(false)
	}

	int32 FRigidGeometryCollectionAsyncPT::GetNumBodies() const
	{
		return Proxy->GetNumTransforms();
	}

	FRigidGeometryCollectionBodyAsyncGT* FRigidGeometryCollectionAsyncPT::GetBody(const UE::Physics::FRigidObjectId& InBodyId) const
	{
		return nullptr;
	}

	FRigidGeometryCollectionBodyAsyncGT* FRigidGeometryCollectionAsyncPT::GetBodyAt(int32 BodyIndex) const
	{
		return nullptr;
	}
}

#endif // UE_RIGIDPHYSICS_API_ENABLED