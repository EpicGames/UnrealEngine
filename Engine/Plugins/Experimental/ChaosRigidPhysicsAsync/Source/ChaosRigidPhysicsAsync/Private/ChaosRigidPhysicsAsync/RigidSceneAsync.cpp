// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/RigidSceneAsync.h"
#include "ChaosRigidPhysicsAsync/RigidGeometryCollectionAsync.h"
#include "Chaos/ChaosScene.h"
#include "RigidPhysics/RigidContext.h"
#include "RigidPhysics/RigidModifier.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidSceneAsyncGT, UE::Physics::FRigidScene);

	FRigidSceneAsyncGT::FRigidSceneAsyncGT(
		const UE::Physics::FRigidDebugName& InName,
		const FRigidSceneSettingsAsync& InSceneSettings)
		: UE::Physics::FRigidScene()
		, Name(InName)
		, SceneSettings(InSceneSettings)
		, Scene(nullptr)
		, Owner()
		, BodyPool(10)
		, SceneModifiers()
		, ScenePT()
		, bOwnsScene(false)
	{
		ConstructScene();
	}

	FRigidSceneAsyncGT::~FRigidSceneAsyncGT()
	{
		DestructScene();
	}

	FPBDRigidsSolver* FRigidSceneAsyncGT::GetSolver() const
	{
		if (Scene != nullptr)
		{
			return Scene->GetSolver();
		}
		return nullptr;
	}

	void FRigidSceneAsyncGT::ConstructScene()
	{
		using namespace UE::Physics;

		FString StaticName = FString::Format(TEXT("{0} Static"), { Name.Value() });

		ScenePT = MakeUnique<FRigidSceneAsyncPT>(Name);
	}

	void FRigidSceneAsyncGT::DestructScene()
	{
		using namespace UE::Physics;

		ScenePT.Reset();

		FRigidScene::DestructScene();

		BodyPool.Reset();
	}

	void FRigidSceneAsyncGT::Startup()
	{
		using namespace UE::Physics;

		if (Scene == nullptr)
		{
			Scene = new FChaosScene(Owner.Get(), SceneSettings.AsyncDt, FName(*Name.Value()));
			bOwnsScene = true;
		}

		ScenePT->Startup(GetHandle(), Scene->GetSolver());
	}

	void FRigidSceneAsyncGT::Shutdown()
	{
		ScenePT->Shutdown();

		if (Scene != nullptr)
		{
			Scene->Flush();

			if (bOwnsScene)
			{
				delete Scene;
			}
		}

		Scene = nullptr;
		bOwnsScene = false;
	}

	void FRigidSceneAsyncGT::RegisterBody(UE::Physics::IRigidBody* InBody)
	{
		const FRigidObjectId BodyId = GetBodyRegistry()->AddObject(InBody);
		InBody->SetId(BodyId);
	}

	void FRigidSceneAsyncGT::UnRegisterBody(const UE::Physics::FRigidBodyHandle& InBodyHandle, const UE::Physics::IRigidBody* InBody)
	{
		GetBodyRegistry()->RemoveObjectChecked(InBodyHandle.GetId(), InBody);
	}

	void FRigidSceneAsyncGT::RegisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer)
	{
		using namespace UE::Physics;

		FRigidObjectId ContainerId = GetBodyContainerRegistry()->AddObject(InContainer);
		InContainer->SetId(ContainerId);
	}

	void FRigidSceneAsyncGT::UnregisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer)
	{
		using namespace UE::Physics;

		GetBodyContainerRegistry()->RemoveObject(InContainer->GetHandle().GetId());
		InContainer->SetId(FRigidObjectId());
	}

	UE::Physics::FRigidSceneId FRigidSceneAsyncGT::GetId() const
	{
		return SceneId;
	}

	void FRigidSceneAsyncGT::SetId(const UE::Physics::FRigidSceneId InId)
	{
		SceneId = InId;
	}

	UE::Physics::FRigidSceneHandle FRigidSceneAsyncGT::GetHandle() const
	{
		return UE::Physics::FRigidSceneHandle(const_cast<FRigidSceneAsyncGT*>(this));
	}

	void FRigidSceneAsyncGT::Lock(UE::Physics::ERigidLockType InLockType) const
	{
		using namespace UE::Physics;

		UE_RIGIDPHYSICS_CHECK(IsInGameThreadContext());

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			switch (InLockType)
			{
			case ERigidLockType::ReadOnly:
				Solver->GetExternalDataLock_External().ReadLock();
				break;
			case ERigidLockType::ReadWrite:
				Solver->GetExternalDataLock_External().WriteLock();
				break;
			}
		}
	}

	void FRigidSceneAsyncGT::Unlock(UE::Physics::ERigidLockType InLockType) const
	{
		using namespace UE::Physics;

		UE_RIGIDPHYSICS_CHECK(IsInGameThreadContext());

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			switch (InLockType)
			{
			case ERigidLockType::ReadOnly:
				Solver->GetExternalDataLock_External().ReadUnlock();
				break;
			case ERigidLockType::ReadWrite:
				Solver->GetExternalDataLock_External().WriteUnlock();
				break;
			}
		}
	}

	void FRigidSceneAsyncGT::ActivateBody(FRigidBodyAsyncGT* InBody)
	{
		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			if (!InBody->IsActive())
			{
				UE_RIGIDPHYSICS_CHECK(InBody->GetProxy() != nullptr);

				// NOTE: ChaosScene handles both GT and PT Particles.
				TArray<FPhysicsActorHandle> ActorHandles;
				ActorHandles.Add(InBody->GetProxy());
				Scene->AddActorsToScene_AssumesLocked(ActorHandles);

				// NOTE: Currently this does nothing because Solver->RegisterObject which is called by
				// AddActorsToScene_AssumesLocked will activate the particle via the dirty proxy mechanism.
				Solver->EnqueueCommandImmediate(
					[BodyHandle = InBody->GetHandle(), this]()
					{
						ScenePT->ActivateBody(BodyHandle);
					});
			}
		}
	}

	void FRigidSceneAsyncGT::DeactivateBody(FRigidBodyAsyncGT* InBody)
	{
		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			if (InBody->IsActive())
			{
				UE_RIGIDPHYSICS_CHECK(InBody->GetProxy() != nullptr);

				Solver->EnqueueCommandImmediate(
					[BodyHandle = InBody->GetHandle(), this]()
					{
						ScenePT->DeactivateBody(BodyHandle);
					});

				// No FChaosScene::RemoveActorsFromScene_AssumesLocked, but it should do this...
				FChaosEngineInterface::RemoveActorFromSolver(InBody->GetProxy(), Solver);
				Scene->RemoveActorFromAccelerationStructure(InBody->GetProxy());
			}
		}
	}

	FRigidBodyAsyncGT* FRigidSceneAsyncGT::CreateBody(const UE::Physics::FRigidDebugName& InName, UE::Physics::ERigidMovementType InMovementType)
	{
		using namespace UE::Physics;
		CheckGameWriteLocked();

		FRigidBodyAsyncGT* BodyGT = BodyPool.Alloc(this, InName, InMovementType);
		UE_RIGIDPHYSICS_CHECK(BodyGT != nullptr);

		if (BodyGT != nullptr)
		{
			RegisterBody(BodyGT);

			if (FPBDRigidsSolver* Solver = GetSolver())
			{
				Solver->EnqueueCommandImmediate(
					[this, BodyHandle = BodyGT->GetHandle(), Proxy = BodyGT->GetProxy()]()
					{
						ScenePT->CreateBodyPT(BodyHandle, Proxy);
					});
			}
		}

		return BodyGT;
	}

	void FRigidSceneAsyncGT::DestroyBody(UE::Physics::IRigidBody* InBody)
	{
		using namespace UE::Physics;
		CheckGameWriteLocked();

		UE_RIGIDPHYSICS_CHECK(InBody != nullptr);

		FRigidBodyAsyncGT* Body = InBody->AsAChecked<FRigidBodyAsyncGT>();
		UE_RIGIDPHYSICS_CHECK(Body);
		if (Body != nullptr)
		{
			const FRigidBodyHandle BodyHandle = Body->GetHandle();
			if (FPBDRigidsSolver* Solver = GetSolver())
			{
				Solver->EnqueueCommandImmediate(
					[this, BodyHandle]()
					{
						ScenePT->DestroyBodyPT(BodyHandle);
					});
			}

			// If the body had any shapes left, make sure to free them all.
			ShapeInstanceContainer.Free(Body->GetShapeInstanceView());

			Body->Deactivate();
			UnRegisterBody(BodyHandle, Body);
			BodyPool.Free(Body);
		}
	}

	FRigidGeometryCollectionAsyncGT* FRigidSceneAsyncGT::CreateGeometryCollection(const UE::Physics::FRigidDebugName& InName, const FSimulationParameters& InParameters)
	{
		using namespace UE::Physics;

		FRigidGeometryCollectionAsyncGT* GC = new FRigidGeometryCollectionAsyncGT(this, InName, InParameters);

		RegisterBodyContainer(GC);

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->EnqueueCommandImmediate(
				[this, Handle = GC->GetHandle(), Proxy = GC->GetProxy()]()
				{
					ScenePT->CreateGeometryCollectionPT(Handle, Proxy);
				});
		}

		return GC;
	}

	void FRigidSceneAsyncGT::DestroyGeometryCollection(UE::Physics::IRigidGeometryCollection* InGC)
	{
		if (FRigidGeometryCollectionAsyncGT* GC = InGC->AsAChecked<FRigidGeometryCollectionAsyncGT>())
		{
			if (FPBDRigidsSolver* Solver = GetSolver())
			{
				Solver->EnqueueCommandImmediate(
					[this, Handle = GC->GetHandle()]()
					{
						ScenePT->DestroyGeometryCollectionPT(Handle);
					});
			}

			UnregisterBodyContainer(GC);

			delete GC;
		}
	}

	void FRigidSceneAsyncGT::AddDisabledCollisions(const TMap<UE::Physics::FRigidBodyHandle, TArray<UE::Physics::FRigidBodyHandle>>& InMap)
	{
		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			TMap<FPhysicsActorHandle, TArray<FPhysicsActorHandle>> IgnoreMap;

			for (const TPair<UE::Physics::FRigidBodyHandle, TArray<UE::Physics::FRigidBodyHandle>>& Pair : InMap)
			{
				if(UE::Physics::IRigidBody* Key = GetBody(Pair.Key.GetId()))
				{ 
					FRigidBodyAsyncGT* KeyBody = Key->AsAChecked<FRigidBodyAsyncGT>();

					const TArray<UE::Physics::FRigidBodyHandle>& Values = Pair.Value;

					TArray<FPhysicsActorHandle> Handles;
					Handles.Reserve(Values.Num());

					for (const UE::Physics::FRigidBodyHandle& Value : Values)
					{
						if(UE::Physics::IRigidBody* ValuePtr = GetBody(Value.GetId()))
						{ 
							FRigidBodyAsyncGT* ValueBody = ValuePtr->AsAChecked<FRigidBodyAsyncGT>();
							Handles.Add(ValueBody->GetProxy());
						}
					}

					IgnoreMap.Add(KeyBody->GetProxy(), Handles);
				}
			}

			FChaosEngineInterface::AddDisabledCollisionsFor_AssumesLocked(IgnoreMap);
		}
	}

	FRigidShapeInstanceAsync* FRigidSceneAsyncGT::AllocateShape()
	{
		return ShapeInstanceContainer.Allocate(SceneId);
	}

	void FRigidSceneAsyncGT::DestroyShape(FRigidShapeInstanceAsync* InShape)
	{
		ShapeInstanceContainer.Free(InShape);
	}

	FRigidShapeInstanceAsync* FRigidSceneAsyncGT::GetShape(const FRigidObjectId& InId)  const
	{
		return ShapeInstanceContainer.Find(InId);
	}

	void FRigidSceneAsyncGT::AddShape(FRigidBodyAsyncGT& Body, FRigidShapeInstanceAsync& InShapeInstance)
	{
		const FRigidBodyHandle BodyHandle = Body.GetHandle();
		const FRigidShapeInstanceHandle ShapeInstanceHandle = InShapeInstance.GetHandle();
		FShapeInstanceView& View = Body.GetShapeInstanceView();
		const int32 ShapeIndex = ShapeInstanceContainer.Add(InShapeInstance, View);

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->EnqueueCommandImmediate(
				[this, BodyHandle, ShapeInstanceHandle, ShapeIndex]()
				{
					ScenePT->AddShapePT(BodyHandle, ShapeInstanceHandle, ShapeIndex);
				});
		}
	}

	void FRigidSceneAsyncGT::RemoveShape(FRigidBodyAsyncGT& Body, FRigidShapeInstanceAsync& InShapeInstance)
	{
		const FRigidBodyHandle BodyHandle = Body.GetHandle();
		FShapeInstanceView& View = Body.GetShapeInstanceView();
		FRigidShapeInstanceHandle ShapeInstanceHandle = InShapeInstance.GetHandle();

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->EnqueueCommandImmediate(
				[this, BodyHandle, ShapeInstanceHandle]()
				{
					ScenePT->RemoveShapePT(BodyHandle, ShapeInstanceHandle);
				});
		}

		ShapeInstanceContainer.Remove(InShapeInstance, View);
	}

	FRigidShapeInstanceAsync* FRigidSceneAsyncGT::GetShape(const FShapeInstanceView& InView, const int32 InIndex) const
	{
		return ShapeInstanceContainer.Get(InView, InIndex);
	}

	FJointConstraint6DOFAsyncGT* FRigidSceneAsyncGT::CreateJointConstraint6DOF()
	{
		FJointConstraint6DOFAsyncGT* Constraint = new FJointConstraint6DOFAsyncGT(this);
		const FRigidObjectId Id = GetJointConstraintRegistry()->AddObject(Constraint);
		Constraint->SetId(Id);

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			// Note: These need to be cached outside the command as the GT object can get destroyed before the command is executed.
			const FJointConstraint6DOFHandle Handle = Constraint->GetTypedHandle();
			Chaos::FJointConstraint* ChaosJointConstraint = Constraint->GetJointConstraint();
			Solver->EnqueueCommandImmediate([this, Handle, ChaosJointConstraint]()
				{
					ScenePT->CreateJointConstraint6DOFPT(Handle, ChaosJointConstraint);
				});
		}
		return Constraint;
	}

	void FRigidSceneAsyncGT::DestroyJointConstraint(IJointConstraint* InConstraint)
	{
		if (FJointConstraint6DOFAsyncGT* Constraint6DOF = InConstraint->AsAChecked<FJointConstraint6DOFAsyncGT>())
		{
			const FJointConstraint6DOFHandle Handle = Constraint6DOF->GetTypedHandle();
			if (FPBDRigidsSolver* Solver = GetSolver())
			{
				Solver->EnqueueCommandImmediate(
					[this, Handle]()
					{
						ScenePT->DestroyJointConstraintPT(Handle);
					});
			}

			GetJointConstraintRegistry()->RemoveObject(Handle.GetId());
			delete Constraint6DOF;
		}
	}

	IJointConstraint* FRigidSceneAsyncGT::GetJointConstraint(const FRigidObjectId& InConstraintId) const
	{
		return GetJointConstraintRegistry()->FindObject(InConstraintId);
	}

	void FRigidSceneAsyncGT::ActivateConstraint(FJointConstraint6DOFAsyncGT* InConstraint)
	{
		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->EnqueueCommandImmediate([this, Solver, InConstraint]()
				{
					Solver->RegisterObject(InConstraint->GetJointConstraint());
				});
		}
	}

	void FRigidSceneAsyncGT::DeactivateConstraint(FJointConstraint6DOFAsyncGT* InConstraint)
	{
		// TODO
	}

	void FRigidSceneAsyncGT::RegisterModifier(UE::Physics::IRigidSceneModifier* Modifier)
	{
		CheckGameWriteLocked();

		SceneModifiers.Add(Modifier);

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->EnqueueCommandImmediate(
				[Modifier, this]()
				{
					ScenePT->RegisterModifier(Modifier);
				});
		}
	}

	void FRigidSceneAsyncGT::UnregisterModifier(UE::Physics::IRigidSceneModifier* Modifier)
	{
		CheckGameWriteLocked();

		SceneModifiers.Remove(Modifier);

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->EnqueueCommandImmediate(
			[Modifier, this]()
			{
				ScenePT->UnregisterModifier(Modifier);
			});
		}
	}

	bool FRigidSceneAsyncGT::Query(const UE::Physics::FRigidQueryDesc& QueryDesc, TArray<UE::Physics::FRigidQueryHit>& OutHits)
	{
		CheckGameReadLocked();

		// TODO: Runs the query on the GT
		return false;
	}

	void FRigidSceneAsyncGT::StartTick(const float InDt)
	{
		CheckGameWriteLocked();

		OnPreSimulate(InDt);

		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->AdvanceAndDispatch_External(InDt);
		}
	}

	void FRigidSceneAsyncGT::WaitOnTick()
	{
		CheckGameWriteLocked();
		
		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->WaitOnPendingTasks_External();
		}
	}

	void FRigidSceneAsyncGT::EndTick()
	{
		CheckGameWriteLocked();

		WaitOnTick();
		
		// TODO: CopySolverAccelerationStructure
		if (FPBDRigidsSolver* Solver = GetSolver())
		{
			Solver->UpdateGameThreadStructures();
			// TODO:
			// - FlipEventManagerBuffer
			// - SyncEvents_GameThread
			// - Concrete.SyncQueryMaterials_External();
		}

		OnPostSimulate();
		// TODO: broadcast old OnPhysScenePostTick?
	}

	void FRigidSceneAsyncGT::OnRigidBodyTransformUpdated(FRigidBodyAsyncGT* Body)
	{
		UpdateRigidBodyInAccelerationStructure(Body);
	}

	void FRigidSceneAsyncGT::UpdateRigidBodyInAccelerationStructure(FRigidBodyAsyncGT* Body)
	{
		using namespace Chaos;

		if ((Body != nullptr) && (Body->GetProxy() != nullptr) && (Scene != nullptr))
		{
			if (Body->IsActive())
			{
				Scene->UpdateActorInAccelerationStructure(Body->GetProxy());
			}
		}
	}

	void FRigidSceneAsyncGT::CheckGameReadLocked() const
	{
		UE_RIGIDPHYSICS_CHECK(IsInGameThreadContext());

		// TODO_CHAOSAPI: Check read or write lock
		//if (Solver != nullptr)
		//{
		//	Solver->GetExternalDataLock_External().IsReadLocked();
		//}
	}

	void FRigidSceneAsyncGT::CheckGameWriteLocked() const
	{
		UE_RIGIDPHYSICS_CHECK(IsInGameThreadContext());

		// TODO_CHAOSAPI: Check write lock
		//if (Solver != nullptr)
		//{
		//	Solver->GetExternalDataLock_External().IsWriteLocked();
		//}
	}

	void FRigidSceneAsyncGT::OnPreSimulate(FReal Dt)
	{
		using namespace UE::Physics;

		CheckGameWriteLocked();

		FRigidContextGameRW Context = FRigidContextGameRW(this);
		for (IRigidSceneModifier* SceneModifier : SceneModifiers)
		{
			SceneModifier->PreSimulate(Context);
		}
	}

	void FRigidSceneAsyncGT::OnPostSimulate()
	{
		using namespace UE::Physics;

		CheckGameWriteLocked();

		FRigidContextGameRW Context = FRigidContextGameRW(this);
		for (IRigidSceneModifier* SceneModifier : SceneModifiers)
		{
			SceneModifier->PostSimulate(Context);
		}
	}
}


namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidSceneAsyncPT, UE::Physics::IRigidScene);

	FRigidSceneAsyncPT::FRigidSceneAsyncPT(const UE::Physics::FRigidDebugName& InName)
		: Name(InName)
		, Solver(nullptr)
		, BodyPool(10)
		, SceneModifiers()
		, bIsStarted(false)
	{
	}

	FRigidSceneAsyncPT::~FRigidSceneAsyncPT()
	{
		UE_RIGIDPHYSICS_CHECK(BodyContainerRegistry.IsEmpty());
	}

	void FRigidSceneAsyncPT::Lock(UE::Physics::ERigidLockType InLockType) const
	{
	}

	void FRigidSceneAsyncPT::Unlock(UE::Physics::ERigidLockType InLockType) const
	{
	}
	
	UE::Physics::FRigidSceneId FRigidSceneAsyncPT::GetId() const
	{
		return SceneHandle.GetId();
	}

	UE::Physics::FRigidSceneHandle FRigidSceneAsyncPT::GetHandle() const
	{
		return SceneHandle;
	}

	void FRigidSceneAsyncPT::RegisterModifier(UE::Physics::IRigidSceneModifier* Modifier)
	{
		CheckSimContext();
		SceneModifiers.Add(Modifier);
	}

	void FRigidSceneAsyncPT::UnregisterModifier(UE::Physics::IRigidSceneModifier* Modifier)
	{
		CheckSimContext();
		SceneModifiers.Remove(Modifier);
	}

	bool FRigidSceneAsyncPT::Query(const UE::Physics::FRigidQueryDesc& QueryDesc, TArray<UE::Physics::FRigidQueryHit>& OutHits)
	{
		CheckSimContext();

		// TODO: Runs the query on the PT

		return false;
	}

	UE::Physics::FRigidSceneHandle FRigidSceneAsyncPT::GetSceneHandle() const
	{
		CheckSimContext();
		return SceneHandle;
	}

	void FRigidSceneAsyncPT::Startup(UE::Physics::FRigidSceneHandle InSceneHandle, FPBDRigidsSolver* InSolver)
	{
		CheckGameContext();

		SceneHandle = InSceneHandle;

		Solver = InSolver;

		bIsStarted = true;

		Solver->GetEvolution()->SetPreIntegrateCallback(
			[this](FReal Dt)
			{
				OnPreTick(Dt);
			});

		Solver->GetEvolution()->SetPostSolveCallback(
			[this](FReal Dt)
			{
				OnPostTick(Dt);
			});
	}

	void FRigidSceneAsyncPT::Shutdown()
	{
		CheckGameContext();

		if (Solver != nullptr)
		{
			Solver->GetEvolution()->SetPreIntegrateCallback([](FReal Dt) {});
			Solver->GetEvolution()->SetPostSolveCallback([](FReal Dt) {});
		}

		bIsStarted = false;
	}

	void FRigidSceneAsyncPT::RegisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer)
	{
		UE_RIGIDPHYSICS_CHECK(InContainer != nullptr);
		UE_RIGIDPHYSICS_CHECK(GetBodyContainer(InContainer->GetHandle().GetId()) == nullptr);

		BodyContainerRegistry.Add(InContainer->GetHandle().GetId(), InContainer);
	}

	void FRigidSceneAsyncPT::UnregisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer)
	{
		UE_RIGIDPHYSICS_CHECK(InContainer != nullptr);
		UE_RIGIDPHYSICS_CHECK(GetBodyContainer(InContainer->GetHandle().GetId()) == InContainer);

		BodyContainerRegistry.Remove(InContainer->GetHandle().GetId());
	}

	void FRigidSceneAsyncPT::CreateGeometryCollectionPT(const UE::Physics::FRigidBodyContainerHandle& InHandle, FGeometryCollectionPhysicsProxy* InProxy)
	{
		UE_RIGIDPHYSICS_CHECK(GetBodyContainer(InHandle.GetId()) == nullptr);

		FRigidGeometryCollectionAsyncPT* GC = new FRigidGeometryCollectionAsyncPT(this, InHandle.GetId(), InProxy);
		RegisterBodyContainer(GC);
	}

	void FRigidSceneAsyncPT::DestroyGeometryCollectionPT(const UE::Physics::FRigidBodyContainerHandle& InHandle)
	{
		if (FRigidGeometryCollectionAsyncPT* GC = GetGeometryCollection(InHandle.GetId()))
		{
			UnregisterBodyContainer(GC);
			delete GC;
		}
	}

	void FRigidSceneAsyncPT::ActivateBody(const UE::Physics::FRigidBodyHandle& InBodyHandle)
	{
		CheckSimContext();
	}

	void FRigidSceneAsyncPT::DeactivateBody(const UE::Physics::FRigidBodyHandle& InBodyHandle)
	{
		CheckSimContext();
	}

	void FRigidSceneAsyncPT::CreateBodyPT(const UE::Physics::FRigidBodyHandle& InBodyHandle, FSingleParticlePhysicsProxy* InProxy)
	{
		CheckSimContext();
		UE_RIGIDPHYSICS_CHECK(GetBody(InBodyHandle.GetId()) == nullptr);

		FRigidBodyAsyncPT* Body = BodyPool.Alloc(this, InBodyHandle, InProxy);
		RegisterBody(InBodyHandle, Body);
	}

	void FRigidSceneAsyncPT::DestroyBodyPT(const UE::Physics::FRigidBodyHandle& InBodyHandle)
	{
		using UE::Physics::IRigidBodyContainer;

		UE::Physics::IRigidBody* Body = GetBody(InBodyHandle.GetId());
		UnRegisterBody(InBodyHandle, Body);
		if (FRigidBodyAsyncPT* TypedBody = Body->AsAChecked<FRigidBodyAsyncPT>())
		{
			ShapeInstanceContainer.Free(TypedBody->GetShapeInstanceView());

			BodyPool.Free(TypedBody);
		}
	}
	
	void FRigidSceneAsyncPT::RegisterBody(const UE::Physics::FRigidBodyHandle& InBodyHandle, UE::Physics::IRigidBody* InBody)
	{
		BodyRegistry.Add(InBodyHandle.GetId(), InBody);
	}

	void FRigidSceneAsyncPT::UnRegisterBody(const UE::Physics::FRigidBodyHandle& InBodyHandle, const UE::Physics::IRigidBody* InBody)
	{
		UE_RIGIDPHYSICS_CHECK(GetBody(InBodyHandle.GetId()) == InBody);
		BodyRegistry.Remove(InBodyHandle.GetId());
	}
	
	UE::Physics::IRigidBody* FRigidSceneAsyncPT::GetBody(const UE::Physics::FRigidObjectId& InBodyId) const
	{
		using namespace UE::Physics;
		CheckSimContext();

		if (IRigidBody*const* BodyValuePtr = BodyRegistry.Find(InBodyId))
		{
			return *BodyValuePtr;
		}

		return nullptr;
	}

	UE::Physics::IRigidBodyContainer* FRigidSceneAsyncPT::GetBodyContainer(const UE::Physics::FRigidObjectId& InBodyContainerId) const
	{
		using namespace UE::Physics;
		CheckSimContext();

		if (IRigidBodyContainer*const* BodyContainerValuePtr = BodyContainerRegistry.Find(InBodyContainerId))
		{
			return *BodyContainerValuePtr;
		}

		return nullptr;
	}

	FRigidGeometryCollectionAsyncPT* FRigidSceneAsyncPT::GetGeometryCollection(const UE::Physics::FRigidObjectId& InGeometryCollectionId) const
	{
		using namespace UE::Physics;
		CheckSimContext();

		if (IRigidBodyContainer* BodyContainer = GetBodyContainer(InGeometryCollectionId))
		{
			return BodyContainer->AsA<FRigidGeometryCollectionAsyncPT>();
		}

		return nullptr;
	}

	FRigidShapeInstanceAsync* FRigidSceneAsyncPT::GetShape(const FRigidObjectId& InId) const
	{
		CheckSimContext();

		return ShapeInstanceContainer.Find(InId);
	}

	void FRigidSceneAsyncPT::AddShapePT(const FRigidBodyHandle BodyHandle, const FRigidShapeInstanceHandle ShapeInstanceHandle, const int32 ShapeIndex)
	{
		CheckSimContext();

		if (UE::Physics::IRigidBody* Body = GetBody(BodyHandle.GetId()))
		{
			if (FRigidBodyAsyncPT* TypedBody = Body->AsAChecked<FRigidBodyAsyncPT>())
			{
				if (TypedBody->IsValid())
				{
					FRigidShapeInstanceAsync* ShapeInstance = ShapeInstanceContainer.Allocate(ShapeInstanceHandle);
					FShapeInstanceView& View = TypedBody->GetShapeInstanceView();

					ShapeInstanceContainer.Add(*ShapeInstance, ShapeIndex, View);
					TypedBody->UpdateShapeInstance(*ShapeInstance, ShapeIndex);
				}
			}
		}
	}

	void FRigidSceneAsyncPT::RemoveShapePT(const FRigidBodyHandle BodyHandle, const FRigidShapeInstanceHandle ShapeInstanceHandle)
	{
		CheckSimContext();

		if (UE::Physics::IRigidBody* Body = GetBody(BodyHandle.GetId()))
		{
			if (FRigidBodyAsyncPT* TypedBody = Body->AsAChecked<FRigidBodyAsyncPT>())
			{
				if (TypedBody->IsValid())
				{
					FShapeInstanceView& View = TypedBody->GetShapeInstanceView();
					FRigidShapeInstanceAsync* ShapeInstance = GetShape(ShapeInstanceHandle.GetId());

					// Remove from the body's shape view
					ShapeInstanceContainer.Remove(*ShapeInstance, View);

					// Update all shape instances to point at the correct shape data (PT rebuilds from "scratch" so old data gets invalidated)
					for (int32 I = 0; I < View.Num(); ++I)
					{
						TypedBody->UpdateShapeInstance(*GetShape(View, I), I);
					}
					// Free the shape
					ShapeInstanceContainer.Free(ShapeInstance);
				}
			}
		}
	}

	FRigidShapeInstanceAsync* FRigidSceneAsyncPT::GetShape(const FShapeInstanceView& InView, const int32 InIndex) const
	{
		CheckSimContext();

		return ShapeInstanceContainer.Get(InView, InIndex);
	}
	
	FJointConstraint6DOFAsyncPT* FRigidSceneAsyncPT::CreateJointConstraint6DOFPT(const FJointConstraint6DOFHandle& InHandle, Chaos::FJointConstraint* InJointConstraint)
	{
		CheckSimContext();

		FJointConstraint6DOFAsyncPT* Result = new FJointConstraint6DOFAsyncPT(InHandle, this, InJointConstraint);
		JointConstraintRegistry.Add(InHandle.GetId(), Result);
		return Result;
	}

	void FRigidSceneAsyncPT::DestroyJointConstraintPT(const FJointConstraint6DOFHandle& InHandle)
	{
		if (IJointConstraint** JointConstraintPtr = JointConstraintRegistry.Find(InHandle.GetId()))
		{
			IJointConstraint* JointConstraint = *JointConstraintPtr;
			JointConstraintRegistry.Remove(InHandle.GetId());
			delete JointConstraint;
		}
	}

	IJointConstraint* FRigidSceneAsyncPT::GetJointConstraint(const FRigidObjectId& InConstraintId) const
	{
		CheckSimContext();

		if (IJointConstraint*const* JointPtr = JointConstraintRegistry.Find(InConstraintId))
		{
			return *JointPtr;
		}
		return nullptr;
	}

	void FRigidSceneAsyncPT::ActiveJointConstraint(const FJointConstraint6DOFHandle& InHandle)
	{
		CheckSimContext();
	}

	void FRigidSceneAsyncPT::DeactivateJointConstraint(const FJointConstraint6DOFHandle& InHandle)
	{
		CheckSimContext();
	}

	void FRigidSceneAsyncPT::CheckGameContext() const
	{
		UE_RIGIDPHYSICS_CHECK(IsInGameThreadContext());
	}

	void FRigidSceneAsyncPT::CheckSimContext() const
	{
		UE_RIGIDPHYSICS_CHECK(!bIsStarted || IsInPhysicsThreadContext());
	}

	void FRigidSceneAsyncPT::OnPreTick(FReal Dt)
	{
		using namespace UE::Physics;

		CheckSimContext();

		FRigidContextSimRW Context = FRigidContextSimRW(this);
		for (IRigidSceneModifier* SceneModifier : SceneModifiers)
		{
			SceneModifier->PreTick(Context);
		}

		// Handle legacy modifiers
		Solver->OnEvolutionPreIntegrate(Dt);
	}

	void FRigidSceneAsyncPT::OnPostTick(FReal Dt)
	{
		using namespace UE::Physics;

		CheckSimContext();

		FRigidContextSimRW Context = FRigidContextSimRW(this);
		for (IRigidSceneModifier* SceneModifier : SceneModifiers)
		{
			SceneModifier->PostTick(Context);
		}

		// Handle legacy modifiers
		Solver->OnEvolutionPostIntegrate(Dt);
	}

}

#endif // UE_RIGIDPHYSICS_API_ENABLED