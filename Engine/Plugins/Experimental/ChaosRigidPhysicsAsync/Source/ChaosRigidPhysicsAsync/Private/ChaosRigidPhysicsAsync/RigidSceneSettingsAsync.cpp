// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::Rigids::Async
{
	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidSceneSettingsAsync, UE::Physics::IRigidSceneSettings);
}

#endif
