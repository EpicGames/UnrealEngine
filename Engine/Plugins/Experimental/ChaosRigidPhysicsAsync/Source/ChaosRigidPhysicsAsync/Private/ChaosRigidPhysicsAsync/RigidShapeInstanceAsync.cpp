// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/RigidShapeInstanceAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/Box.h"
#include "Chaos/CastingUtilities.h"
#include "Chaos/Capsule.h"
#include "Chaos/Convex.h"
#include "Chaos/ImplicitObjectTransformed.h"
#include "Chaos/ShapeInstance.h"
#include "Chaos/Sphere.h"
#include "Chaos/HeightField.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "ChaosRigidPhysicsAsync/Conversions.h"

namespace Chaos::Rigids::Async
{
	using FBoxGeometry = UE::Physics::FBoxGeometry;
	using FCapsuleGeometry = UE::Physics::FCapsuleGeometry;
	using FSphereGeometry = UE::Physics::FSphereGeometry;
	using FConvexGeometry = UE::Physics::FConvexGeometry;
	using FTriangleMeshGeometry = UE::Physics::FTriangleMeshGeometry;
	using FHeightFieldGeometry = UE::Physics::FHeightFieldGeometry;
	using FAnyGeometry = UE::Physics::FAnyGeometry;
	using FRigidShapeInstanceSetup = UE::Physics::FRigidShapeInstanceSetup;
	using FImplicitTransformed = Chaos::TImplicitObjectTransformed<FReal, 3>;

	UE_RIGIDPHYSICS_RIGIDTYPED_IMPL(FRigidShapeInstanceAsync, UE::Physics::IRigidShapeInstance);

	Chaos::FImplicitObjectPtr MakeImplicit(const FBoxGeometry& Geometry, const FRigidShapeInstanceSetup& InSetup)
	{
		const FTransform3f& InLocalTransform = InSetup.LocalTransform;
		const FVector InScale(InLocalTransform.GetScale3D());
		const FQuat InRotation(InLocalTransform.GetRotation());
		const FVector InTranslation(InLocalTransform.GetTranslation());

		float MinScale, MinScaleAbs;
		FVector Scale3DAbs;
		SetupNonUniformHelper(InScale, MinScale, MinScaleAbs, Scale3DAbs);

		const FVector3f ScaledExtents = Geometry.GetExtents() * FVector3f(Scale3DAbs);
		Chaos::FVec3 HalfExtents = ScaledExtents * 0.5;
		HalfExtents.X = FMath::Max(HalfExtents.X, UE_KINDA_SMALL_NUMBER);
		HalfExtents.Y = FMath::Max(HalfExtents.Y, UE_KINDA_SMALL_NUMBER);
		HalfExtents.Z = FMath::Max(HalfExtents.Z, UE_KINDA_SMALL_NUMBER);

		if (!InRotation.IsIdentity())
		{
			const FTransform LocalTransform(InRotation, InTranslation);
			Chaos::FImplicitObjectPtr BoxImpl = MakeImplicitObjectPtr<Chaos::FImplicitBox3>(-HalfExtents, HalfExtents);
			return MakeImplicitObjectPtr<Chaos::FImplicitObjectTransformed>(MoveTemp(BoxImpl), LocalTransform);
		}
		const Chaos::FVec3 Center(InTranslation);
		return MakeImplicitObjectPtr<Chaos::FImplicitBox3>(Center - HalfExtents, Center + HalfExtents);
	}

	Chaos::FImplicitObjectPtr MakeImplicit(const FSphereGeometry& Geometry, const FRigidShapeInstanceSetup& InSetup)
	{
		const FTransform3f& InLocalTransform = InSetup.LocalTransform;
		const FVector InScale(InLocalTransform.GetScale3D());

		float MinScale, MinScaleAbs;
		FVector Scale3DAbs;
		SetupNonUniformHelper(InScale, MinScale, MinScaleAbs, Scale3DAbs);

		const float ScaledRadius = Geometry.GetRadius() * MinScaleAbs;
		const Chaos::FVec3 Center(InLocalTransform.GetTranslation());
		return MakeImplicitObjectPtr<Chaos::FSphere>(Center, ScaledRadius);
	}

	Chaos::FImplicitObjectPtr MakeImplicit(const FCapsuleGeometry& Geometry, const FRigidShapeInstanceSetup& InSetup)
	{
		const FTransform3f& InLocalTransform = InSetup.LocalTransform;
		const FQuat InRotation(InLocalTransform.GetRotation());
		const FVector InTranslation(InLocalTransform.GetTranslation());

		float MinScale, MinScaleAbs;
		FVector Scale3DAbs;
		SetupNonUniformHelper(FVector(InLocalTransform.GetScale3D()), MinScale, MinScaleAbs, Scale3DAbs);

		const float Radius = Geometry.GetRadius();
		const float SegmentHalfLen = Geometry.GetSegmentHalfLength();
		const float RadiusScale = FMath::Max(Scale3DAbs.X, Scale3DAbs.Y);
		const float LengthScale = Scale3DAbs.Z;

		const float ScaledHalfLength = FMath::Max((Radius + SegmentHalfLen) * LengthScale, 0.1);
		const float ScaledRadius = FMath::Clamp(RadiusScale * Radius, 0.1f, ScaledHalfLength);
		const float FinalSegmentHalfLen = FMath::Max(0.05f, ScaledHalfLength - ScaledRadius);

		const Chaos::FVec3 Axis = InRotation.RotateVector(FVector::ZAxisVector);
		const Chaos::FVec3 Center(InTranslation);
		const Chaos::FVec3 X1 = Center - Axis * FinalSegmentHalfLen;
		const Chaos::FVec3 X2 = Center + Axis * FinalSegmentHalfLen;
		return MakeImplicitObjectPtr<Chaos::FCapsule>(X1, X2, ScaledRadius);
	}

	Chaos::FImplicitObjectPtr MakeImplicit(const FConvexGeometry& Geometry, const FRigidShapeInstanceSetup& InSetup)
	{
		const Chaos::FConvexPtr ConvexImplicit = Geometry.GetImplicit();
		if (!ensure(ConvexImplicit))
		{
			return nullptr;
		}

		const FTransform3f& InLocalTransform = InSetup.LocalTransform;
		const FVector InScale(InLocalTransform.GetScale3D());
		const FQuat InRotation(InLocalTransform.GetRotation());
		const FVector InTranslation(InLocalTransform.GetTranslation());

		// Clamp near-zero scale components to avoid degenerate scaled shapes
		FVector NetScale = InScale;
		NetScale.X = FMath::Abs(NetScale.X) < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : NetScale.X;
		NetScale.Y = FMath::Abs(NetScale.Y) < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : NetScale.Y;
		NetScale.Z = FMath::Abs(NetScale.Z) < UE_KINDA_SMALL_NUMBER ? UE_KINDA_SMALL_NUMBER : NetScale.Z;

		// Extract the scale from the transform - we have separate wrapper classes for scale versus translate/rotate
		const FTransform ConvexTransform(InRotation, InTranslation, FVector(1, 1, 1));
		const bool bHasTranslationOrRotation = !ConvexTransform.GetTranslation().IsNearlyZero() || !ConvexTransform.GetRotation().IsIdentity();
		const bool bNoScale = FVector::PointsAreNear(NetScale, FVector(1), UE_KINDA_SMALL_NUMBER);

		// Wrap in Instanced (identity scale) or Scaled depending on scale value
		// NOTE: margin lives on the wrapper, not the inner shared FConvex object
		Chaos::FImplicitObjectPtr Implicit;
		if (bNoScale)
		{
			Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectInstanced<Chaos::FConvex>>(ConvexImplicit);
		}
		else
		{
			Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectScaled<Chaos::FConvex>>(ConvexImplicit, NetScale);
		}

		// Wrap in a transform if translation or rotation is non-trivial (scale is pulled out above)
		if (bHasTranslationOrRotation)
		{
			Implicit = MakeImplicitObjectPtr<FImplicitTransformed>(MoveTemp(Implicit), ConvexTransform);
		}

		return Implicit;
	}

	Chaos::FImplicitObjectPtr MakeImplicit(const FTriangleMeshGeometry& Geometry, const FRigidShapeInstanceSetup& InSetup)
	{
		Chaos::FTriangleMeshImplicitObjectPtr TrimeshImplicit = Geometry.GetImplicit();
		if (!ensure(TrimeshImplicit))
		{
			return nullptr;
		}

		const FTransform3f& InLocalTransform = InSetup.LocalTransform;
		const FVector InScale(InLocalTransform.GetScale3D());
		const FQuat InRotation(InLocalTransform.GetRotation());
		const FVector InTranslation(InLocalTransform.GetTranslation());

		const FTransform MeshTransform = FTransform(InRotation, InTranslation, FVector(1, 1, 1));
		const bool bHasTranslationOrRotation = !MeshTransform.GetTranslation().IsNearlyZero() || !MeshTransform.GetRotation().IsIdentity();

		// Extract the scale from the transform - we have separate wrapper classes for scale versus translate/rotate 
		const bool bNoScale = FVector::PointsAreNear(InScale, FVector(1), UE_KINDA_SMALL_NUMBER);

		Chaos::FImplicitObjectPtr Implicit;
		if (bNoScale)
		{
			Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectInstanced<Chaos::FTriangleMeshImplicitObject>>(TrimeshImplicit);
		}
		else
		{
			Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>>(TrimeshImplicit, InScale);
		}

		// Wrap the mesh in a non-scaled transform if necessary (the scale is pulled out above)
		if (bHasTranslationOrRotation)
		{
			Implicit = MakeImplicitObjectPtr<FImplicitObjectTransformed>(MoveTemp(Implicit), MeshTransform);
		}
		return Implicit;
	}

	Chaos::FImplicitObjectPtr MakeImplicit(const FHeightFieldGeometry& Geometry, const FRigidShapeInstanceSetup& InSetup)
	{
		const TRefCountPtr<Chaos::FHeightField> HeightFieldImplicit = Geometry.GetImplicit();
		if (!ensure(HeightFieldImplicit))
		{
			return nullptr;
		}

		const FTransform3f& InLocalTransform = InSetup.LocalTransform;
		const FVector InScale(InLocalTransform.GetScale3D());
		const FQuat InRotation(InLocalTransform.GetRotation());
		const FVector InTranslation(InLocalTransform.GetTranslation());

		const FTransform MeshTransform = FTransform(InRotation, InTranslation, FVector(1, 1, 1));
		const bool bHasTranslationOrRotation = !MeshTransform.GetTranslation().IsNearlyZero() || !MeshTransform.GetRotation().IsIdentity();
		const bool bNoScale = FVector::PointsAreNear(InScale, FVector(1), UE_KINDA_SMALL_NUMBER);

		Chaos::FImplicitObjectPtr Implicit;
		if (bNoScale)
		{
			Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectInstanced<Chaos::FHeightField>>(HeightFieldImplicit);
		}
		else
		{
			Implicit = MakeImplicitObjectPtr<Chaos::TImplicitObjectScaled<Chaos::FHeightField>>(HeightFieldImplicit, InScale);
		}

		if (bHasTranslationOrRotation)
		{
			Implicit = MakeImplicitObjectPtr<FImplicitTransformed>(MoveTemp(Implicit), MeshTransform);
		}
		return Implicit;
	}

	template <typename GeometryType>
	Chaos::FImplicitObjectPtr MakeImplicit(const GeometryType& Geometry, const FRigidShapeInstanceSetup& InSetup)
	{
		// Note: Using GeometryType in the static assert so it's unable to be evaluated until template instantiation.
		static_assert(std::is_same_v<GeometryType, void>, "Unsupported shape type");
		return nullptr;
	}

	Chaos::FPerShapeData* FRigidShapeInstanceAsync::Create(const FRigidShapeInstanceSetup& InSetup)
	{
		const FAnyGeometry& InGeometry = InSetup.Geometry;

		Chaos::FImplicitObjectPtr ImplicitPtr = InGeometry.Visit([&InSetup]<typename GeometryType>(const GeometryType& Geometry)
		{
			return MakeImplicit(Geometry, InSetup);
		});

		if (!ensure(ImplicitPtr))
		{
			return nullptr;
		}

		TArray<Chaos::FMaterialHandle> MaterialHandles;
		MaterialHandles.Reserve(InSetup.Materials.Num());
		for (const UE::Physics::FMaterialHandle& Handle : InSetup.Materials)
		{
			MaterialHandles.Add(Handle.GetMaterialHandle());
		}
		TArray<Chaos::FMaterialMaskHandle> MaskHandles;
		MaskHandles.Reserve(InSetup.MaterialMaskHandles.Num());
		for (const UE::Physics::FMaterialMaskHandle& Handle : InSetup.MaterialMaskHandles)
		{
			MaskHandles.Add(Handle.GetMaterialMaskHandle());
		}
		TArray<Chaos::FMaterialHandle> MaskMapMaterialHandles;
		MaskMapMaterialHandles.Reserve(InSetup.MaterialMaskMapMaterials.Num());
		for (const UE::Physics::FMaterialHandle& Handle : InSetup.MaterialMaskMapMaterials)
		{
			MaskMapMaterialHandles.Add(Handle.GetMaterialHandle());
		}

		TUniquePtr<Chaos::FPerShapeData> NewShape = Chaos::FShapeInstanceProxy::Make(0, ImplicitPtr);
		Chaos::FPerShapeData* PerShapeData = NewShape.Release();

		PerShapeData->SetQueryEnabled(InSetup.bEnableQuery);
		PerShapeData->SetSimEnabled(InSetup.bEnableSim);
		PerShapeData->SetIsProbe(InSetup.bEnableProbe);
		PerShapeData->SetShapeFilterData(InSetup.ShapeFilterData);
		PerShapeData->SetFilterInstanceData(InSetup.FilterInstanceData);
		PerShapeData->SetCollisionTraceType(ConvertCollisionTraceFlag(InSetup.CollisionTraceType));
		// TODO: UpdateShapeBounds?
		PerShapeData->SetMaterials(MoveTemp(MaterialHandles));
		PerShapeData->SetMaterialMasks(MoveTemp(MaskHandles));
		PerShapeData->SetMaterialMaskMaps(InSetup.MaterialMaskMaps);
		PerShapeData->SetMaterialMaskMapMaterials(MoveTemp(MaskMapMaterialHandles));
		PerShapeData->SetUserData(InSetup.UserData);

		return PerShapeData;
	}

	UE::Physics::FRigidShapeInstanceHandle FRigidShapeInstanceAsync::GetHandle() const
	{
		return Handle;
	}

	UE::Physics::FAnyGeometry FRigidShapeInstanceAsync::GetGeometry() const
	{
		FImplicitObject* ImplicitGeom = PerShapeData->GetGeometry();
		auto EmptyCallback = []<typename ImplicitType>(ImplicitType& Implicit){};
		const EImplicitObjectType ImplicitType = GetLeafImplicit(ImplicitGeom, EmptyCallback);

		switch (ImplicitType)
		{
			case ImplicitObjectType::Sphere:
			{
				return FSphereGeometry(ImplicitGeom->GetObjectChecked<FImplicitSphere3>().GetRadiusf());
			}
			case ImplicitObjectType::Box:
			{
				return FBoxGeometry(FVector3f(ImplicitGeom->GetObjectChecked<FImplicitBox3>().Extents()));
			}
			case ImplicitObjectType::Capsule:
			{
				const FImplicitCapsule3& Capsule = ImplicitGeom->GetObjectChecked<FImplicitCapsule3>();
				return FCapsuleGeometry(Capsule.GetRadiusf(), Capsule.GetHeightf() / 2.0f);
			}
			case ImplicitObjectType::Convex:
			{
				FConvex& Convex = ImplicitGeom->GetObjectChecked<FConvex>();
				return UE::Physics::FConvexGeometry(&Convex);
			}
			case ImplicitObjectType::TriangleMesh:
			{
				FTriangleMeshImplicitObject& TriMesh = ImplicitGeom->GetObjectChecked<FTriangleMeshImplicitObject>();
				return UE::Physics::FTriangleMeshGeometry(&TriMesh);
			}
			case ImplicitObjectType::HeightField:
			{
				FHeightField& HeightField = ImplicitGeom->GetObjectChecked<FHeightField>();
				return UE::Physics::FHeightFieldGeometry(&HeightField);
			}
			default:
			{
				ensureMsgf(false, TEXT("Unhandled implicit leaf type"));
				break;
			}
		}
		return FAnyGeometry();
	}

	FTransform3f FRigidShapeInstanceAsync::GetLocalTransform() const
	{
		FTransform3f Transform;
		auto AccumulateInternalTransformCallback = [&Transform]<typename ImplicitType>(ImplicitType& Implicit)
		{
			if constexpr (std::is_same_v<ImplicitType, FImplicitTransformed>)
			{
				FTransform3f ImplicitTransform = FTransform3f(Implicit.GetTransform());
				FTransform3f::Multiply(&Transform, &ImplicitTransform, &Transform);
			}
			else if constexpr (std::is_same_v<ImplicitType, FImplicitObjectScaled>)
			{
				FTransform3f ImplicitTransform;
				ImplicitTransform.SetScale3D(FVector3f(Implicit.GetScale()));
				FTransform3f::Multiply(&Transform, &ImplicitTransform, &Transform);
			}
		};

		// Walk the hierarchy to get a leaf implicit, accumulating the transform as we go down.
		FImplicitObjectRef ImplicitGeom = PerShapeData->GetGeometry();
		const EImplicitObjectType ImplicitType = GetLeafImplicit(ImplicitGeom, AccumulateInternalTransformCallback);

		// Now accumulate the transform of the leaf object.
		// For most types there's commonly only an internal or leaf transform, however nothing technically prevents both from existing.
		switch (ImplicitType)
		{
			case ImplicitObjectType::Sphere:
			{
				const FImplicitSphere3& Sphere = ImplicitGeom->GetObjectChecked<FImplicitSphere3>();
				FTransform3f LeafTransform(Sphere.GetCenterf());
				FTransform3f::Multiply(&Transform, &LeafTransform, &Transform);
				break;
			}
			case ImplicitObjectType::Box:
			{
				const FImplicitBox3& Box = ImplicitGeom->GetObjectChecked<FImplicitBox3>();
				FTransform3f LeafTransform(FVector3f(Box.Center()));
				FTransform3f::Multiply(&Transform, &LeafTransform, &Transform);
				break;
			}
			case ImplicitObjectType::Capsule:
			{
				const FImplicitCapsule3& Capsule = ImplicitGeom->GetObjectChecked<FImplicitCapsule3>();
				const FQuat4f Rotation = FQuat4f::FindBetween(FVector3f(0, 0, 1), Capsule.GetAxisf());
				FTransform3f LeafTransform(Rotation, Capsule.GetCenterf());
				FTransform3f::Multiply(&Transform, &LeafTransform, &Transform);
				break;
			}
			case ImplicitObjectType::Convex:
			{
				break;
			}
			case ImplicitObjectType::TriangleMesh:
			{
				break;
			}
			case ImplicitObjectType::HeightField:
			{
				break;
			}
			default:
			{
				ensureMsgf(false, TEXT("Unhandled implicit leaf type"));
				break;
			}
		}
		return Transform;
	}

	bool FRigidShapeInstanceAsync::GetQueryEnabled() const
	{
		return PerShapeData->GetQueryEnabled();
	}

	void FRigidShapeInstanceAsync::SetQueryEnabled(const bool bInEnabled)
	{
		PerShapeData->SetQueryEnabled(bInEnabled);
	}

	bool FRigidShapeInstanceAsync::GetSimEnabled() const
	{
		return PerShapeData->GetSimEnabled();
	}

	void FRigidShapeInstanceAsync::SetSimEnabled(const bool bInEnabled)
	{
		PerShapeData->SetSimEnabled(bInEnabled);
	}

	bool FRigidShapeInstanceAsync::GetIsProbe() const
	{
		return PerShapeData->GetIsProbe();
	}

	void FRigidShapeInstanceAsync::SetIsProbe(const bool bInIsProbe)
	{
		PerShapeData->SetIsProbe(bInIsProbe);
	}

	ECollisionTraceFlag FRigidShapeInstanceAsync::GetCollisionTraceType() const
	{
		return ConvertCollisionTraceFlag(PerShapeData->GetCollisionTraceType());
	}

	void FRigidShapeInstanceAsync::SetCollisionTraceType(const ECollisionTraceFlag InTraceFlag)
	{
		PerShapeData->SetCollisionTraceType(ConvertCollisionTraceFlag(InTraceFlag));
	}

	Chaos::Filter::FShapeFilterData FRigidShapeInstanceAsync::GetShapeFilter() const
	{
		return PerShapeData->GetShapeFilterData();
	}

	void FRigidShapeInstanceAsync::SetShapeFilter(const FShapeFilterData& InShapeFilter)
	{
		PerShapeData->SetShapeFilterData(InShapeFilter);
	}

	Chaos::Filter::FInstanceData FRigidShapeInstanceAsync::GetFilterInstanceData() const
	{
		return PerShapeData->GetFilterInstanceData();
	}

	void FRigidShapeInstanceAsync::SetFilterInstanceData(const Chaos::Filter::FInstanceData& InInstanceData)
	{
		PerShapeData->SetFilterInstanceData(InInstanceData);
	}
	
	int32 FRigidShapeInstanceAsync::GetNumMaterials() const
	{
		return PerShapeData->NumMaterials();
	}

	UE::Physics::FMaterialHandle FRigidShapeInstanceAsync::GetMaterial(const int32 InIndex) const
	{
		return UE::Physics::FMaterialHandle(PerShapeData->GetMaterial(InIndex));
	}

	void FRigidShapeInstanceAsync::SetMaterials(TArray<UE::Physics::FMaterialHandle>&& InMaterials)
	{
		TArray<Chaos::FMaterialHandle> MaterialHandles;
		MaterialHandles.Reserve(InMaterials.Num());
		for (const UE::Physics::FMaterialHandle& MaterialHandle : InMaterials)
		{
			MaterialHandles.Add(MaterialHandle.GetMaterialHandle());
		}
		PerShapeData->SetMaterials(MoveTemp(MaterialHandles));
	}

	int32 FRigidShapeInstanceAsync::GetNumMaterialMasks() const
	{
		return PerShapeData->GetMaterialMasks().Num();
	}

	UE::Physics::FMaterialMaskHandle FRigidShapeInstanceAsync::GetMaterialMask(const int32 InIndex) const
	{
		return UE::Physics::FMaterialMaskHandle(PerShapeData->GetMaterialMasks()[InIndex]);
	}

	void FRigidShapeInstanceAsync::SetMaterialMasks(TArray<UE::Physics::FMaterialMaskHandle>&& InMaterialMasks)
	{
		TArray<Chaos::FMaterialMaskHandle> MaskHandles;
		MaskHandles.Reserve(InMaterialMasks.Num());
		for (const UE::Physics::FMaterialMaskHandle& MaterialMaskHandle : InMaterialMasks)
		{
			MaskHandles.Add(MaterialMaskHandle.GetMaterialMaskHandle());
		}
		PerShapeData->SetMaterialMasks(MoveTemp(MaskHandles));
	}

	int32 FRigidShapeInstanceAsync::GetNumMaterialMaskMaps() const
	{
		return PerShapeData->GetMaterialMaskMaps().Num();
	}

	uint32 FRigidShapeInstanceAsync::GetMaterialMaskMap(const int32 InIndex) const
	{
		return PerShapeData->GetMaterialMaskMaps()[InIndex];
	}

	void FRigidShapeInstanceAsync::SetMaterialMaskMaps(TArray<uint32>&& InMaterialMaskMaps)
	{
		PerShapeData->SetMaterialMaskMaps(MoveTemp(InMaterialMaskMaps));
	}

	int32 FRigidShapeInstanceAsync::GetNumMaterialMaskMapMaterials() const
	{
		return PerShapeData->GetMaterialMaskMapMaterials().Num();
	}

	UE::Physics::FMaterialHandle FRigidShapeInstanceAsync::GetMaterialMaskMapMaterial(const int32 InIndex) const
	{
		return UE::Physics::FMaterialHandle(PerShapeData->GetMaterialMaskMapMaterials()[InIndex]);
	}

	void FRigidShapeInstanceAsync::SetMaterialMaskMapMaterials(TArray<UE::Physics::FMaterialHandle>&& InMaterialMaskMapMaterials)
	{
		TArray<Chaos::FMaterialHandle> MaskMapMaterialHandles;
		MaskMapMaterialHandles.Reserve(InMaterialMaskMapMaterials.Num());
		for (const UE::Physics::FMaterialHandle& MaterialHandle : InMaterialMaskMapMaterials)
		{
			MaskMapMaterialHandles.Add(MaterialHandle.GetMaterialHandle());
		}
		PerShapeData->SetMaterialMaskMapMaterials(MoveTemp(MaskMapMaterialHandles));
	}

	void* FRigidShapeInstanceAsync::GetUserData() const
	{
		return PerShapeData->GetUserData();
	}

	void FRigidShapeInstanceAsync::SetUserData(void* InUserData)
	{
		PerShapeData->SetUserData(InUserData);
	}

	template <typename Lambda>
	EImplicitObjectType FRigidShapeInstanceAsync::GetLeafImplicit(FImplicitObject*& InOutImplicit, const Lambda& Func)
	{
		EImplicitObjectType ImplicitType = InOutImplicit->GetType();
		while (true)
		{
			if (ImplicitType == ImplicitObjectType::Transformed)
			{
				FImplicitTransformed& Transformed = InOutImplicit->GetObjectChecked<FImplicitTransformed>();
				Func(Transformed);
				InOutImplicit = Transformed.MObject.GetReference();
				ImplicitType = InOutImplicit->GetType();
				continue;
			}
			else if (Chaos::IsInstanced(ImplicitType))
			{
				FImplicitObjectInstanced* Instanced = InOutImplicit->AsAChecked<FImplicitObjectInstanced>();
				Func(*Instanced);
				InOutImplicit = Instanced->GetInnerObjectInternal();
				ImplicitType = InOutImplicit->GetType();
				continue;
			}
			else if (Chaos::IsScaled(ImplicitType))
			{
				FImplicitObjectScaled* Scaled = InOutImplicit->AsAChecked<FImplicitObjectScaled>();
				Func(*Scaled);
				InOutImplicit = Scaled->GetInnerObjectInternal();
				ImplicitType = InOutImplicit->GetType();
				continue;
			}
			break;
		}
		return ImplicitType;
	}
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
