// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosRigidPhysicsAsync/ShapeInstanceContainerAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidLog.h"

namespace Chaos::Rigids::Async
{
	int32 FShapeInstanceView::Num() const
	{
		return Ids.Num();
	}

	FShapeInstanceContainerAsyncGT::FShapeInstanceContainerAsyncGT()
		: Pool(10)
		, Registry(10)
	{
	}

	FRigidShapeInstanceAsync* FShapeInstanceContainerAsyncGT::Allocate(const FRigidSceneId InSceneId)
	{
		FRigidShapeInstanceAsync* Result = Pool.Alloc();
		const FRigidObjectId Id = Registry.AddObject(Result);
		Result->Handle = UE::Physics::FRigidShapeInstanceHandle(InSceneId, Id);
		return Result;
	}

	void FShapeInstanceContainerAsyncGT::Free(FRigidShapeInstanceAsync* InInstance)
	{
		Registry.RemoveObjectChecked(InInstance->Handle.GetId(), InInstance);
		Pool.Free(InInstance);
	}

	FRigidShapeInstanceAsync* FShapeInstanceContainerAsyncGT::Find(const FRigidObjectId& InId) const
	{
		return Registry.FindObject(InId);
	}

	int32 FShapeInstanceContainerAsyncGT::Add(FRigidShapeInstanceAsync& InInstance, FShapeInstanceView& InOutView)
	{
		UE_RIGIDPHYSICS_CHECK(InInstance.bAttachedToBody == false);
		InInstance.bAttachedToBody = true;
		InOutView.Ids.Add(InInstance.Handle.GetId());
		return InOutView.Ids.Num() - 1;
	}

	void FShapeInstanceContainerAsyncGT::Remove(FRigidShapeInstanceAsync& InInstance, FShapeInstanceView& InOutView)
	{
		UE_RIGIDPHYSICS_CHECK(InInstance.bAttachedToBody == true);
		InInstance.bAttachedToBody = false;
		InOutView.Ids.Remove(InInstance.Handle.GetId());
	}

	FRigidShapeInstanceAsync* FShapeInstanceContainerAsyncGT::Get(const FShapeInstanceView& InView, const int32 InIndex) const
	{
		const bool bIsValidIndex = InView.Ids.IsValidIndex(InIndex);
		ensure(bIsValidIndex);
		return bIsValidIndex ? Find(InView.Ids[InIndex]) : nullptr;
	}

	void FShapeInstanceContainerAsyncGT::Free(FShapeInstanceView& InView)
	{
		for (int32 I = 0; I < InView.Num(); ++I)
		{
			FRigidShapeInstanceAsync* ShapeInstance = Find(InView.Ids[I]);
			UE_RIGIDPHYSICS_CHECK(ShapeInstance);
			Free(ShapeInstance);
		}
		InView.Ids.Reset();
	}

	FShapeInstanceContainerAsyncPT::FShapeInstanceContainerAsyncPT()
		: Pool(10)
	{
	}

	FRigidShapeInstanceAsync* FShapeInstanceContainerAsyncPT::Allocate(const FRigidShapeInstanceHandle& InHandle)
	{
		UE_RIGIDPHYSICS_CHECK(InHandle.GetId().IsValid());

		FRigidShapeInstanceAsync* Result = Pool.Alloc();
		Result->Handle = InHandle;

		const FRigidObjectId& Id = InHandle.GetId();
		const int32 ArrayIndex = Id.GetArrayIndex();
		Registry.SetNum(ArrayIndex + 1);
		FRegistryItem& Item = Registry[ArrayIndex];
		// Make sure the slot is empty. If it isn't that means some GT->PT communication issue happened.
		UE_RIGIDPHYSICS_CHECK(Item.Object == nullptr);
		Item.Object = Result;
		Item.Epoch = Id.GetEpoch();

		return Result;
	}

	void FShapeInstanceContainerAsyncPT::Free(FRigidShapeInstanceAsync* InInstance)
	{
		const FRigidShapeInstanceHandle& Handle = InInstance->Handle;
		const FRigidObjectId& Id = Handle.GetId();
		const int32 ArrayIndex = Id.GetArrayIndex();

		UE_RIGIDPHYSICS_CHECK(Registry.IsValidIndex(ArrayIndex));
		FRegistryItem& Item = Registry[ArrayIndex];
		UE_RIGIDPHYSICS_CHECK(Item.Object == InInstance);
		UE_RIGIDPHYSICS_CHECK(Item.Epoch == Id.GetEpoch());
		Item = FRegistryItem();

		Pool.Free(InInstance);
	}

	FRigidShapeInstanceAsync* FShapeInstanceContainerAsyncPT::Find(const FRigidObjectId& InId) const
	{
		const int32 ArrayIndex = InId.GetArrayIndex();
		if (Registry.IsValidIndex(ArrayIndex))
		{
			const FRegistryItem& Item = Registry[ArrayIndex];
			if (Item.Epoch == InId.GetEpoch())
			{
				return Item.Object;
			}
		}
		return nullptr;
	}

	void FShapeInstanceContainerAsyncPT::Add(FRigidShapeInstanceAsync& InInstance, const int32 InShapeIndex, FShapeInstanceView& InOutView)
	{
		// All adds should be happening sequentially from the GT. That means each add should always have the index that is the next to add.
		UE_RIGIDPHYSICS_CHECK(InOutView.Num() == InShapeIndex);
		UE_RIGIDPHYSICS_CHECK(InInstance.bAttachedToBody == false);
		InInstance.bAttachedToBody = true;
		InOutView.Ids.Add(InInstance.Handle.GetId());
	}

	void FShapeInstanceContainerAsyncPT::Remove(FRigidShapeInstanceAsync& InInstance, FShapeInstanceView& InOutView)
	{
		UE_RIGIDPHYSICS_CHECK(InInstance.bAttachedToBody == true);
		InInstance.bAttachedToBody = false;
		InOutView.Ids.Remove(InInstance.Handle.GetId());
	}

	FRigidShapeInstanceAsync* FShapeInstanceContainerAsyncPT::Get(const FShapeInstanceView& InView, const int32 InIndex) const
	{
		const bool bIsValidIndex = InView.Ids.IsValidIndex(InIndex);
		ensure(bIsValidIndex);
		return bIsValidIndex ? Find(InView.Ids[InIndex]) : nullptr;
	}

	void FShapeInstanceContainerAsyncPT::Free(FShapeInstanceView& InView)
	{
		for (int32 I = 0; I < InView.Num(); ++I)
		{
			FRigidShapeInstanceAsync* ShapeInstance = Find(InView.Ids[I]);
			check(ShapeInstance);
			Free(ShapeInstance);
		}
		InView.Ids.Reset();
	}
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
