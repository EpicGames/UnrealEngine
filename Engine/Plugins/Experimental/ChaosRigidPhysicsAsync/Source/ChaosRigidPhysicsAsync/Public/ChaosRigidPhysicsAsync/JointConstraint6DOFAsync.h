// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "ChaosRigidPhysicsAsync/RigidFwdAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/JointConstraint6DOF.h"

namespace Chaos
{
	class FJointConstraint;
} // namespace Chaos

namespace Chaos::Rigids::Async
{
	class UE_INTERNAL FJointConstraint6DOFAsyncGT : public UE::Physics::IJointConstraint6DOF
	{
		using EJointMotionType = UE::Physics::EJointMotionType;
		using IRigidBody = UE::Physics::IRigidBody;

	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FJointConstraint6DOFAsyncGT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FJointConstraint6DOFAsyncGT(FRigidSceneAsyncGT* InScene);
		UE_INTERNAL virtual ~FJointConstraint6DOFAsyncGT();

		UE_INTERNAL virtual FJointConstraintHandle GetHandle() const override final;

		UE_INTERNAL virtual void Activate() override final;
		UE_INTERNAL virtual void SetBodies(IRigidBody* Body0, IRigidBody* Body1) override final;
		UE_INTERNAL virtual void SetJointTransforms(const FTransform3d& InTransform0, const FTransform3d& InTransform1) override final;

		UE_INTERNAL virtual double GetLinearLimit() const override final;
		UE_INTERNAL virtual void SetLinearLimit(const double InLinearLimit) override final;

		UE_INTERNAL virtual EJointMotionType GetLinearMotionTypesX() const override final;
		UE_INTERNAL virtual void SetLinearMotionTypesX(const EJointMotionType InMotionType) override final;

		UE_INTERNAL virtual EJointMotionType GetLinearMotionTypesY() const override final;
		UE_INTERNAL virtual void SetLinearMotionTypesY(const EJointMotionType InMotionType) override final;

		UE_INTERNAL virtual EJointMotionType GetLinearMotionTypesZ() const override final;
		UE_INTERNAL virtual void SetLinearMotionTypesZ(const EJointMotionType InMotionType) override final;

		UE_INTERNAL virtual void SetAngularMotionTypesX(const EJointMotionType InMotionType) override final;
		UE_INTERNAL virtual void SetAngularMotionTypesY(const EJointMotionType InMotionType) override final;
		UE_INTERNAL virtual void SetAngularMotionTypesZ(const EJointMotionType InMotionType) override final;

		UE_INTERNAL virtual void SetAngularLimits(const FVector3d& InAngularLimits) override final;

		UE_INTERNAL virtual void SetCollisionEnabled(const bool bCollisionEnabled) override final;
		UE_INTERNAL virtual void SetProjectionEnabled(const bool bProjectionEnabled) override final;
		UE_INTERNAL virtual void SetMassConditioningEnabled(const bool bMassConditioningEnabled) override final;
		UE_INTERNAL virtual void SetUseLinearSolver(const bool bUseLinearSolver) override final;

		UE_INTERNAL virtual void SetSoftLinearLimitsEnabled(const bool bSoftLinearLimitsEnabled) override final;
		UE_INTERNAL virtual void SetSoftTwistLimitsEnabled(const bool bSoftTwistLimitsEnabled) override final;
		UE_INTERNAL virtual void SetSoftSwingLimitsEnabled(const bool bSoftSwingLimitsEnabled) override final;

		UE_INTERNAL virtual void SetSoftLinearStiffness(const double InSoftLinearStiffness) override final;
		UE_INTERNAL virtual void SetSoftLinearDamping(const double InSoftLinearDamping) override final;
		UE_INTERNAL virtual void SetSoftTwistStiffness(const double InSoftTwistStiffness) override final;
		UE_INTERNAL virtual void SetSoftTwistDamping(const double InSoftTwistDamping) override final;
		UE_INTERNAL virtual void SetSoftSwingStiffness(const double InSoftSwingStiffness) override final;
		UE_INTERNAL virtual void SetSoftSwingDamping(const double InSoftSwingDamping) override final;

		UE_INTERNAL virtual void SetProjectionLinearAlpha(const double InProjectionLinearAlpha) override final;
		UE_INTERNAL virtual void SetProjectionAngularAlpha(const double InProjectionAngularAlpha) override final;
		UE_INTERNAL virtual void SetProjectionLinearTolerance(const double InProjectionLinearTolerance) override final;
		UE_INTERNAL virtual void SetProjectionAngularTolerance(const double InProjectionAngularTolerance) override final;

		UE_INTERNAL virtual void SetLinearContactDistance(const double InLinearContactDistance) override final;
		UE_INTERNAL virtual void SetTwistContactDistance(const double InTwistContactDistance) override final;
		UE_INTERNAL virtual void SetSwingContactDistance(const double InSwingContactDistance) override final;

		UE_INTERNAL virtual void SetAngularDriveStiffness(const FVector3d& InAngularDriveStiffness) override final;
		UE_INTERNAL virtual void SetAngularDriveDamping(const FVector3d& InAngularDriveDamping) override final;
		UE_INTERNAL virtual void SetLinearDriveStiffness(const FVector3d& InLinearDriveStiffness) override final;
		UE_INTERNAL virtual void SetLinearDriveDamping(const FVector3d& InLinearDriveDamping) override final;

		UE_INTERNAL void SetId(const FRigidObjectId& InId);
		UE_INTERNAL Chaos::FJointConstraint* GetJointConstraint();
		UE_INTERNAL const Chaos::FJointConstraint* GetJointConstraint() const;

	private:
		UE::Physics::FRigidObjectId ConstraintId;
		FRigidSceneAsyncGT* Scene = nullptr;
		Chaos::FJointConstraint* Constraint = nullptr;
	};

	class UE_INTERNAL FJointConstraint6DOFAsyncPT : public UE::Physics::IJointConstraint6DOF
	{
		using EJointMotionType = UE::Physics::EJointMotionType;

	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FJointConstraint6DOFAsyncPT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FJointConstraint6DOFAsyncPT(const FJointConstraintHandle& InHandle, FRigidSceneAsyncPT* InScene, Chaos::FJointConstraint* InConstraint);
		UE_INTERNAL virtual ~FJointConstraint6DOFAsyncPT() = default;

		UE_INTERNAL virtual FJointConstraintHandle GetHandle() const override final;

		UE_INTERNAL virtual double GetLinearLimit() const override final;
		UE_INTERNAL virtual void SetLinearLimit(const double InLinearLimit) override final;

		UE_INTERNAL virtual EJointMotionType GetLinearMotionTypesX() const override final;
		UE_INTERNAL virtual void SetLinearMotionTypesX(const EJointMotionType InMotionType) override final;

		UE_INTERNAL virtual EJointMotionType GetLinearMotionTypesY() const override final;
		UE_INTERNAL virtual void SetLinearMotionTypesY(const EJointMotionType InMotionType) override final;

		UE_INTERNAL virtual EJointMotionType GetLinearMotionTypesZ() const override final;
		UE_INTERNAL virtual void SetLinearMotionTypesZ(const EJointMotionType InMotionType) override final;

	private:
		FJointConstraintHandle Handle;
		FRigidSceneAsyncPT* Scene = nullptr;
		Chaos::FJointConstraint* Constraint = nullptr;
	};
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
