// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "ChaosRigidPhysicsAsync/RigidFwdAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Logging/StructuredLog.h"
#include "Math/Bounds.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "RigidPhysics/RigidBody.h"
#include "ChaosRigidPhysicsAsync/RigidShapeInstanceAsync.h"
#include "ChaosRigidPhysicsAsync/ShapeInstanceContainerAsync.h"

namespace Chaos::Rigids::Async
{
	//
	//
	// RIGID BODY GT/PT
	//
	// Implemented in terms of the existing FSingleParticlePhysicsProxy.
	// GT and PT bodies implement the IRigidBody API and forward the functions
	// to the appropriate Proxy function. 
	//
	// NOTE: The current Proxy implementation does not sync back changes made in 
	// simulation callbacks if those changes are to data that is not part of the 
	// simulation output (e.g., SetGravityGroup, and most things that are not
	// position, velocity, etc)
	//
	//

	class UE_INTERNAL FRigidBodyAsyncGT : public UE::Physics::IRigidBody
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidBodyAsyncGT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidBodyAsyncGT(FRigidSceneAsyncGT* InScene, const UE::Physics::FRigidDebugName& InName, UE::Physics::ERigidMovementType InMovementType);

		virtual ~FRigidBodyAsyncGT();

		//
		// Begin IRigidBody
		//
		UE_INTERNAL virtual UE::Physics::FRigidBodyHandle GetHandle() const override final;
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API virtual void SetId(const UE::Physics::FRigidObjectId& InId) override final;

		UE_INTERNAL virtual bool IsActive() const override final;
		UE_INTERNAL virtual void Activate() override final;
		UE_INTERNAL virtual void Deactivate() override final;
		UE_INTERNAL virtual bool IsStatic() const override final;
		UE_INTERNAL virtual bool IsKinematic() const override final;
		UE_INTERNAL virtual bool IsDynamic() const override final;
		UE_INTERNAL virtual bool IsSleeping() const override final;

		UE_INTERNAL virtual int32 GetNumShapes() const override final;
		UE_INTERNAL virtual FRigidShapeInstanceAsync* GetShape(int32 InShapeIndex) const override final;
		UE_INTERNAL virtual FRigidShapeInstanceAsync* CreateShape(const UE::Physics::FRigidShapeInstanceSetup& InSetup) override final;
		UE_INTERNAL virtual FRigidShapeInstanceAsync* CreateShape(TUniquePtr<Chaos::FPerShapeData>&& InShapeData) override final;
		UE_INTERNAL virtual void DestroyShape(IRigidShapeInstance* InShape, bool bWakeTouching) override final;
		UE_INTERNAL FShapeInstanceView& GetShapeInstanceView();

		UE_INTERNAL virtual void InitTransform(const FTransform& InTransform) override final;
		UE_INTERNAL virtual void UpdateTransform(const FTransform& InTransform) override final;
		UE_INTERNAL virtual void SetKinematicTarget(const FTransform& InTransform) override final;
		UE_INTERNAL virtual FTransform GetTransform() const override final;
		UE_INTERNAL virtual FBounds3d GetBounds() const override final;
		UE_INTERNAL virtual double GetMass() const override final;
		UE_INTERNAL virtual void SetMass(double InMass) override final;
		UE_INTERNAL virtual FVector3d GetInertia() const override final;
		UE_INTERNAL virtual void SetInertia(const FVector3d& InInertia) override final;
		//UE_INTERNAL virtual void SetCCDEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetMACDEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetSmoothEdgeCollisionsEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetMaxLinearVelocity(float InMax) override final {  }
		//UE_INTERNAL virtual void SetMaxAngularVelocity(float InMax) override final {  }
		//UE_INTERNAL virtual void SetInertiaConditioningEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetGyroscopicTorqueEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetGravityGroup(int32 InId) override final {  }
		UE_INTERNAL virtual void SetCenterOfMass(const FVector3d& InCenterOfMass) override final;
		UE_INTERNAL virtual void SetRotationOfMass(const FQuat& InRotationOfMass) override final;
		UE_INTERNAL virtual void SetGravityScale(float InScale) override final;
		//UE_INTERNAL virtual void SetWakeEventsEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetNumPositionIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetNumVelocityIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetNumProjectionIterations(int32 InNum) override final {  }
		UE_INTERNAL virtual void SetIsSleeping(const bool bInSleeping) override final;
		//
		// End IRigidBody
		//

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API bool IsValid() const;
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API const FSingleParticlePhysicsProxy* GetProxy() const;
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FSingleParticlePhysicsProxy* GetProxy();
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API const FRigidBodyHandle_External& GetProxyAPI() const;
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidBodyHandle_External& GetProxyAPI();
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidSceneAsyncGT* GetScene() const;


	private:
		FRigidShapeInstanceAsync* CreateShapeInternal(Chaos::FPerShapeData* InShapeData);

		UE::Physics::FRigidObjectId BodyId;
		FRigidSceneAsyncGT* SceneAsync = nullptr;
		FSingleParticlePhysicsProxy* Proxy;
		FShapeInstanceView ShapeInstanceView;
	};


	class UE_INTERNAL FRigidBodyAsyncPT : public UE::Physics::IRigidBody
	{
	private:
		inline FRigidBodyHandle_Internal* GetProxyAPI()
		{
			return Proxy->GetPhysicsThreadAPI();
		}

		inline const FRigidBodyHandle_Internal* GetProxyAPI() const
		{
			return Proxy->GetPhysicsThreadAPI();
		}

	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidBodyAsyncPT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidBodyAsyncPT(
			FRigidSceneAsyncPT* InScene,
			const UE::Physics::FRigidBodyHandle& InBodyHandle,
			FSingleParticlePhysicsProxy* InProxy);

		virtual ~FRigidBodyAsyncPT();

		//
		// Begin IRigidBody
		//
		UE_INTERNAL virtual UE::Physics::FRigidBodyHandle GetHandle() const override final;
		UE_INTERNAL virtual bool IsStatic() const override final;
		UE_INTERNAL virtual bool IsKinematic() const override final;
		UE_INTERNAL virtual bool IsDynamic() const override final;
		UE_INTERNAL virtual bool IsSleeping() const override final;

		UE_INTERNAL virtual int32 GetNumShapes() const override final;
		UE_INTERNAL virtual FRigidShapeInstanceAsync* GetShape(int32 InShapeIndex) const override final;
		UE_INTERNAL FShapeInstanceView& GetShapeInstanceView();
		UE_INTERNAL void UpdateShapeInstance(FRigidShapeInstanceAsync& ShapeInstance, const int32 InShapeIndex);

		UE_INTERNAL virtual void InitTransform(const FTransform& InTransform) override final;
		UE_INTERNAL virtual void UpdateTransform(const FTransform& InTransform) override final;
		UE_INTERNAL virtual void SetKinematicTarget(const FTransform& InTransform) override final;
		UE_INTERNAL virtual FTransform GetTransform() const override final;
		UE_INTERNAL virtual FBounds3d GetBounds() const override final;
		UE_INTERNAL virtual double GetMass() const override final;
		UE_INTERNAL virtual void SetMass(double InMass) override final;
		//
		// End IRigidBody
		//

		UE_INTERNAL bool IsValid() const;

	private:
		UE_INTERNAL void SetTransform(const FTransform& InTransform, bool bIsTeleport);
		const FString GetDebugName() const;

		FRigidSceneAsyncPT* Scene = nullptr;
		UE::Physics::FRigidBodyHandle Handle;
		FSingleParticlePhysicsProxy* Proxy;
		FShapeInstanceView ShapeInstanceView;
	};

} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
