// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidFactory.h"
#include "RigidPhysics/RigidSceneSettings.h"

namespace Chaos::Rigids::Async
{
	// TODO_CHAOSAPI: We should be regsitering an instance of FRigidFactoryAsync with
	// some kind of abstract factory system
	//
	class FRigidFactoryAsync : public UE::Physics::IRigidFactory
	{
	public:
		FRigidFactoryAsync();
		virtual ~FRigidFactoryAsync();
		UE_INTERNAL virtual const UE::Physics::FRigidTypeId& GetSceneTypeId() const override final;
		UE_INTERNAL UE::Physics::FRigidSceneHandle CreateScene(const UE::Physics::FRigidDebugName& InName, const UE::Physics::IRigidSceneSettings* InSettings);
		UE_INTERNAL virtual void DestroyScene(const UE::Physics::FRigidSceneHandle& InSceneHandle) override final;

	private:
		TArray<UE::Physics::IRigidScene*> Scenes;
	};
}

#endif // UE_RIGIDPHYSICS_API_ENABLED