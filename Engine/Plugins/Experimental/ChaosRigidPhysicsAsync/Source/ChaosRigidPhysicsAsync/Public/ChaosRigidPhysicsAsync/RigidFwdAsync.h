// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

namespace Chaos::Rigids::Async
{
	class FRigidSceneSettingsAsync;
} // namespace Chaos::Rigids::Async

// Internal fwd decls
namespace Chaos::Rigids::Async
{
	class UE_INTERNAL FJointConstraint6DOFAsyncGT;
	class UE_INTERNAL FJointConstraint6DOFAsyncPT;
	class UE_INTERNAL FRigidBodyAsyncGT;
	class UE_INTERNAL FRigidBodyAsyncPT;
	class UE_INTERNAL FRigidBodyPoolAsyncGT;
	class UE_INTERNAL FRigidBodyPoolAsyncPT;
	class UE_INTERNAL FRigidGeometryCollectionAsyncGT;
	class UE_INTERNAL FRigidGeometryCollectionAsyncPT;
	class UE_INTERNAL FRigidSceneAsyncGT;
	class UE_INTERNAL FRigidSceneAsyncPT;
	class UE_INTERNAL FRigidShapeInstanceAsync;
	
	using UE_INTERNAL FJointConstraintHandle = UE::Physics::FJointConstraintHandle;
	using UE_INTERNAL FJointConstraint6DOFHandle = UE::Physics::FJointConstraint6DOFHandle;
	using UE_INTERNAL FJointConstraintRegistry = UE::Physics::FJointConstraintRegistry;
	using UE_INTERNAL FRigidBodyHandle = UE::Physics::FRigidBodyHandle;
	using UE_INTERNAL FRigidObjectId = UE::Physics::FRigidObjectId;
	using UE_INTERNAL FRigidShapeInstanceHandle = UE::Physics::FRigidShapeInstanceHandle;
	using UE_INTERNAL IJointConstraint = UE::Physics::IJointConstraint;
	using UE_INTERNAL IJointConstraint6DOF = UE::Physics::IJointConstraint6DOF;
	using UE_INTERNAL IRigidShapeInstance = UE::Physics::IRigidShapeInstance;
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
