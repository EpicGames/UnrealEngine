// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosRigidPhysicsAsync/RigidFwdAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "RigidPhysics/RigidBody.h"
#include "RigidPhysics/RigidGeometryCollection.h"


namespace Chaos::Rigids::Async
{
	class UE_INTERNAL FRigidGeometryCollectionBodyAsyncGT : public UE::Physics::IRigidBody
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidGeometryCollectionBodyAsyncGT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidGeometryCollectionBodyAsyncGT(FRigidGeometryCollectionAsyncGT* InGeometryCollection);

		virtual ~FRigidGeometryCollectionBodyAsyncGT();

		//
		// Begin IRigidBody
		//
		UE_INTERNAL virtual UE::Physics::FRigidBodyHandle GetHandle() const override final;
		UE_INTERNAL virtual bool IsActive() const override final;
		UE_INTERNAL virtual void Activate() override final;
		UE_INTERNAL virtual void Deactivate() override final;
		UE_INTERNAL virtual bool IsStatic() const override final;
		UE_INTERNAL virtual bool IsKinematic() const override final;
		UE_INTERNAL virtual bool IsDynamic() const override final;
		UE_INTERNAL virtual bool IsSleeping() const override final;
		UE_INTERNAL virtual int32 GetNumShapes() const override final;
		UE_INTERNAL virtual UE::Physics::IRigidShapeInstance* GetShape(int32 InShapeIndex) const override final;
		//UE_INTERNAL virtual void AddShape(const UE::Physics::IRigidShapeInstance* InCollider) override final;
		//UE_INTERNAL virtual void InitTransform(const FTransform& InTransform) override final;
		//UE_INTERNAL virtual void UpdateTransform(const FTransform& InTransform) override final;
		//UE_INTERNAL virtual void SetKinematicTarget(const FTransform& InTransform) override final;
		//UE_INTERNAL virtual FTransform GetTransform() const override final;
		//UE_INTERNAL virtual FBounds3d GetBounds() const override final;
		//UE_INTERNAL virtual double GetMass() const override final;
		//UE_INTERNAL virtual void SetMass(double InMass) override final;
		//UE_INTERNAL virtual FVector3d GetInertia() const override final;
		//UE_INTERNAL virtual void SetInertia(const FVector3d& InInertia) override final;
		//UE_INTERNAL virtual void SetCCDEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetMACDEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetSmoothEdgeCollisionsEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetMaxLinearVelocity(float InMax) override final {  }
		//UE_INTERNAL virtual void SetMaxAngularVelocity(float InMax) override final {  }
		//UE_INTERNAL virtual void SetInertiaConditioningEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetGyroscopicTorqueEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetGravityGroup(int32 InId) override final {  }
		//UE_INTERNAL virtual void SetGravityScale(float InScale) override final;
		//UE_INTERNAL virtual void SetWakeEventsEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetNumPositionIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetNumVelocityIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetNumProjectionIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetIsSleeping(bool bInSleeping) override final {  }
		//
		// End IRigidBody
		//

	};

	class UE_INTERNAL FRigidGeometryCollectionBodyAsyncPT : public UE::Physics::IRigidBody
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidGeometryCollectionBodyAsyncPT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidGeometryCollectionBodyAsyncPT(FRigidGeometryCollectionAsyncPT* InGeometryCollection);

		virtual ~FRigidGeometryCollectionBodyAsyncPT();

		//
		// Begin IRigidBody
		//
		UE_INTERNAL virtual UE::Physics::FRigidBodyHandle GetHandle() const override final;
		UE_INTERNAL virtual bool IsActive() const override final;
		UE_INTERNAL virtual void Activate() override final;
		UE_INTERNAL virtual void Deactivate() override final;
		UE_INTERNAL virtual bool IsStatic() const override final;
		UE_INTERNAL virtual bool IsKinematic() const override final;
		UE_INTERNAL virtual bool IsDynamic() const override final;
		UE_INTERNAL virtual bool IsSleeping() const override final;
		UE_INTERNAL virtual int32 GetNumShapes() const override final;
		UE_INTERNAL virtual UE::Physics::IRigidShapeInstance* GetShape(int32 InShapeIndex) const override final;
		//UE_INTERNAL virtual void AddShape(const UE::Physics::IRigidShapeInstance* InCollider) override final;
		//UE_INTERNAL virtual void InitTransform(const FTransform& InTransform) override final;
		//UE_INTERNAL virtual void UpdateTransform(const FTransform& InTransform) override final;
		//UE_INTERNAL virtual void SetKinematicTarget(const FTransform& InTransform) override final;
		//UE_INTERNAL virtual FTransform GetTransform() const override final;
		//UE_INTERNAL virtual FBounds3d GetBounds() const override final;
		//UE_INTERNAL virtual double GetMass() const override final;
		//UE_INTERNAL virtual void SetMass(double InMass) override final;
		//UE_INTERNAL virtual FVector3d GetInertia() const override final;
		//UE_INTERNAL virtual void SetInertia(const FVector3d& InInertia) override final;
		//UE_INTERNAL virtual void SetCCDEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetMACDEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetSmoothEdgeCollisionsEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetMaxLinearVelocity(float InMax) override final {  }
		//UE_INTERNAL virtual void SetMaxAngularVelocity(float InMax) override final {  }
		//UE_INTERNAL virtual void SetInertiaConditioningEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetGyroscopicTorqueEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetGravityGroup(int32 InId) override final {  }
		//UE_INTERNAL virtual void SetGravityScale(float InScale) override final;
		//UE_INTERNAL virtual void SetWakeEventsEnabled(bool bInEnabled) override final {  }
		//UE_INTERNAL virtual void SetNumPositionIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetNumVelocityIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetNumProjectionIterations(int32 InNum) override final {  }
		//UE_INTERNAL virtual void SetIsSleeping(bool bInSleeping) override final {  }
		//
		// End IRigidBody
		//

	};

	// The full rigid body API. There will be several implementations of this API. E.g., synchronous
	// physics, asynchronous physics, immediate physics, remote physics, etc.
	// NOTE: for some implementations like async physics, there are both Game- and Physics-Context
	// implementations within the async implementation. Pinning a handle will return the object 
	// appropriate for the context.
	class UE_INTERNAL FRigidGeometryCollectionAsyncGT : public UE::Physics::IRigidGeometryCollection
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidGeometryCollectionAsyncGT);

		FRigidGeometryCollectionAsyncGT(FRigidSceneAsyncGT* InScene, const UE::Physics::FRigidDebugName& InName, const FSimulationParameters& InParameters);
		virtual ~FRigidGeometryCollectionAsyncGT();

		//
		// Begin IRigidBodyContainer
		//
		UE_INTERNAL virtual UE::Physics::FRigidBodyContainerHandle GetHandle() const override final;
		UE_INTERNAL virtual int32 GetNumBodies() const override final;
		UE_INTERNAL virtual FRigidGeometryCollectionBodyAsyncGT* GetBody(const UE::Physics::FRigidObjectId& InBodyId) const override final;
		UE_INTERNAL virtual void SetId(const UE::Physics::FRigidObjectId& InBodyContainerId) override final;
		//
		// End IRigidBodyContainer
		//

		//
		// Begin IRigidGeometryCollection
		//
		UE_INTERNAL virtual FRigidGeometryCollectionBodyAsyncGT* GetBodyAt(int32 BodyIndex) const override final;
		//
		// End IRigidGeometryCollection
		//


	public:
		UE_INTERNAL FGeometryCollectionPhysicsProxy* GetProxy() const
		{
			return Proxy;
		}

	private:
		UE::Physics::FRigidDebugName Name;
		UE::Physics::FRigidObjectId BodyContainerId;
		TSharedPtr<FGeometryCollection> RestCollection;
		FRigidSceneAsyncGT* Scene;
		FGeometryCollectionPhysicsProxy* Proxy;
	};

	class UE_INTERNAL FRigidGeometryCollectionAsyncPT : public UE::Physics::IRigidGeometryCollection
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidGeometryCollectionAsyncPT);

		FRigidGeometryCollectionAsyncPT(FRigidSceneAsyncPT* InScene, const UE::Physics::FRigidObjectId& InId, FGeometryCollectionPhysicsProxy* InProxy);
		virtual ~FRigidGeometryCollectionAsyncPT();

		//
		// Begin IRigidBodyContainer
		//
		UE_INTERNAL virtual UE::Physics::FRigidBodyContainerHandle GetHandle() const override final;
		UE_INTERNAL virtual int32 GetNumBodies() const override final;
		UE_INTERNAL virtual FRigidGeometryCollectionBodyAsyncGT* GetBody(const UE::Physics::FRigidObjectId& InBodyId) const override final;
		UE_INTERNAL virtual void SetId(const UE::Physics::FRigidObjectId& InBodyContainerId) override final;
		//
		// End IRigidBodyContainer
		//

		//
		// Begin IRigidGeometryCollection
		//
		UE_INTERNAL virtual FRigidGeometryCollectionBodyAsyncGT* GetBodyAt(int32 BodyIndex) const override final;
		//
		// End IRigidGeometryCollection
		//

	public:
		UE_INTERNAL FGeometryCollectionPhysicsProxy* GetProxy() const
		{
			return Proxy;
		}

	private:
		UE::Physics::FRigidObjectId BodyContainerId;
		FRigidSceneAsyncPT* Scene;
		FGeometryCollectionPhysicsProxy* Proxy;
	};

}

#endif
