// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosRigidPhysicsAsync/RigidFwdAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/ObjectPool.h"
#include "ChaosRigidPhysicsAsync/JointConstraint6DOFAsync.h"
#include "ChaosRigidPhysicsAsync/RigidBodyAsync.h"
#include "ChaosRigidPhysicsAsync/RigidFactoryAsync.h"
#include "ChaosRigidPhysicsAsync/RigidGeometryCollectionAsync.h"
#include "ChaosRigidPhysicsAsync/RigidSceneSettingsAsync.h"
#include "ChaosRigidPhysicsAsync/RigidShapeInstanceAsync.h"
#include "ChaosRigidPhysicsAsync/ShapeInstanceContainerAsync.h"
#include "ChaosSolversModule.h"
#include "Containers/Map.h"
#include "Logging/StructuredLog.h"
#include "PBDRigidsSolver.h"
#include "Physics/Experimental/PhysScene_Chaos.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "RigidPhysics/RigidScene.h"

class FChaosScene;

namespace Chaos
{
	class FPBDRigidsSolver;
}

namespace Chaos::Rigids::Async
{
	//
	//
	// RIGID SCENE GT/PT
	//
	// The GT/PT scenes are implemented as a wrapper around Chaos::FPBDRigidsSolver
	// (the equivalen of a "SceneProxy"). GT and PT Scenes reference the same solver,
	// and each implements IRigidScene, forwarding the functions to the appropriate
	// solver function for the context.
	//
	//

	class UE_INTERNAL FRigidSceneAsyncGT
		: public UE::Physics::FRigidScene
		, public TSharedFromThis<FRigidSceneAsyncGT, ESPMode::ThreadSafe>
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidSceneAsyncGT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidSceneAsyncGT(
			const UE::Physics::FRigidDebugName& InName,
			const FRigidSceneSettingsAsync& InSceneSettings);
		virtual ~FRigidSceneAsyncGT();

		//
		// Begin IRigidLockable
		//
	protected:
		UE_INTERNAL virtual void Lock(UE::Physics::ERigidLockType InLockType) const override final;
		UE_INTERNAL virtual void Unlock(UE::Physics::ERigidLockType InLockType) const override final;
		//
		// End IRigidLockable
		//

		//
		// IRIGIDSCENE BEGIN
		//
	public:

		UE_INTERNAL virtual UE::Physics::FRigidSceneId GetId() const override final;
		UE_INTERNAL void SetId(const UE::Physics::FRigidSceneId InId);
		UE_INTERNAL virtual UE::Physics::FRigidSceneHandle GetHandle() const override final;
		UE_INTERNAL virtual void RegisterModifier(UE::Physics::IRigidSceneModifier* Modifier) override final;
		UE_INTERNAL virtual void UnregisterModifier(UE::Physics::IRigidSceneModifier* Modifier) override final;
		UE_INTERNAL virtual bool Query(const UE::Physics::FRigidQueryDesc& QueryDesc, TArray<UE::Physics::FRigidQueryHit>& OutHits) override final;
		UE_INTERNAL virtual void StartTick(const float InDt) override final;
		UE_INTERNAL virtual void WaitOnTick() override final;
		UE_INTERNAL virtual void EndTick() override final;
		UE_INTERNAL virtual FRigidBodyAsyncGT* CreateBody(const UE::Physics::FRigidDebugName& InName, UE::Physics::ERigidMovementType InMovementType) override final;
		UE_INTERNAL virtual void DestroyBody(UE::Physics::IRigidBody* InBody) override final;
		UE_INTERNAL virtual FRigidGeometryCollectionAsyncGT* CreateGeometryCollection(const UE::Physics::FRigidDebugName& InName, const FSimulationParameters& InParameters) override final;
		UE_INTERNAL virtual void DestroyGeometryCollection(UE::Physics::IRigidGeometryCollection* InGC) override final;

		UE_INTERNAL virtual void AddDisabledCollisions(const TMap<UE::Physics::FRigidBodyHandle, TArray<UE::Physics::FRigidBodyHandle>>& InMap) override final;
		//
		// IRIGIDSCENE END
		//

		// ShapeInstance
		UE_INTERNAL FRigidShapeInstanceAsync* AllocateShape();
		UE_INTERNAL void DestroyShape(FRigidShapeInstanceAsync* InShape);
		UE_INTERNAL virtual FRigidShapeInstanceAsync* GetShape(const FRigidObjectId& InId) const override final;
		// ShapeInstance: Body
		UE_INTERNAL void AddShape(FRigidBodyAsyncGT& Body, FRigidShapeInstanceAsync& InShapeInstance);
		UE_INTERNAL void RemoveShape(FRigidBodyAsyncGT& Body, FRigidShapeInstanceAsync& InShapeInstance);
		UE_INTERNAL FRigidShapeInstanceAsync* GetShape(const FShapeInstanceView& InView, const int32 InIndex) const;

		// Joint Constraint
		UE_INTERNAL virtual FJointConstraint6DOFAsyncGT* CreateJointConstraint6DOF() override final;
		UE_INTERNAL virtual void DestroyJointConstraint(IJointConstraint* InConstraint) override final;
		UE_INTERNAL virtual IJointConstraint* GetJointConstraint(const FRigidObjectId& InConstraintId) const override final;
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void ActivateConstraint(FJointConstraint6DOFAsyncGT* InConstraint);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void DeactivateConstraint(FJointConstraint6DOFAsyncGT* InConstraint);

	public:
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void Startup();
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void Shutdown();
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void RegisterBody(UE::Physics::IRigidBody* InBody);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void UnRegisterBody(const UE::Physics::FRigidBodyHandle& InBodyHandle, const UE::Physics::IRigidBody* InBody);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void RegisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void UnregisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void ActivateBody(FRigidBodyAsyncGT* InBody);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void DeactivateBody(FRigidBodyAsyncGT* InBody);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void OnRigidBodyTransformUpdated(FRigidBodyAsyncGT* Body);

	private:
		void CheckGameReadLocked() const;
		void CheckGameWriteLocked() const;
		void ConstructScene();
		void DestructScene();
		void OnPreSimulate(FReal Dt);
		void OnPostSimulate();
		void UpdateRigidBodyInAccelerationStructure(FRigidBodyAsyncGT* Body);
		FPBDRigidsSolver* GetSolver() const;

		UE::Physics::FRigidDebugName Name;

		FRigidSceneSettingsAsync SceneSettings;
		UE::Physics::FRigidSceneId SceneId;

		FChaosScene* Scene;
		TStrongObjectPtr<UObject> Owner;

		TObjectPool<FRigidBodyAsyncGT> BodyPool;

		TArray<UE::Physics::IRigidSceneModifier*> SceneModifiers;

		TUniquePtr<FRigidSceneAsyncPT> ScenePT;
		FShapeInstanceContainerAsyncGT ShapeInstanceContainer;

		// TODO_CHAOSAPI: Temp until Scene is always created through new API
		bool bOwnsScene;
	};


	class UE_INTERNAL FRigidSceneAsyncPT : public UE::Physics::IRigidScene
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidSceneAsyncPT);

		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API FRigidSceneAsyncPT(const UE::Physics::FRigidDebugName& InName);

		CHAOSRIGIDPHYSICSASYNC_API virtual ~FRigidSceneAsyncPT();

		//
		// Begin IRigidLockable
		//
	protected:
		UE_INTERNAL virtual void Lock(UE::Physics::ERigidLockType InLockType) const override final;
		UE_INTERNAL virtual void Unlock(UE::Physics::ERigidLockType InLockType) const override final;
		//
		// End IRigidLockable
		//

		//
		// IRIGIDSCENE BEGIN
		//
	public:
		UE_INTERNAL virtual UE::Physics::FRigidSceneId GetId() const override final;
		UE_INTERNAL virtual UE::Physics::FRigidSceneHandle GetHandle() const override final;
		UE_INTERNAL virtual UE::Physics::IRigidBody* GetBody(const UE::Physics::FRigidObjectId& InBodyId) const override final;
		UE_INTERNAL virtual UE::Physics::IRigidBodyContainer* GetBodyContainer(const UE::Physics::FRigidObjectId& InBodyContainerId) const override final;
		UE_INTERNAL virtual FRigidGeometryCollectionAsyncPT* GetGeometryCollection(const UE::Physics::FRigidObjectId& InGeometryCollectionId) const override final;
		UE_INTERNAL virtual void RegisterModifier(UE::Physics::IRigidSceneModifier* Modifier) override final;
		UE_INTERNAL virtual void UnregisterModifier(UE::Physics::IRigidSceneModifier* Modifier) override final;
		UE_INTERNAL virtual bool Query(const UE::Physics::FRigidQueryDesc& QueryDesc, TArray<UE::Physics::FRigidQueryHit>& OutHits) override final;
		//
		// IRIGIDSCENE END
		//

	public:
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API UE::Physics::FRigidSceneHandle GetSceneHandle() const;
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void Startup(UE::Physics::FRigidSceneHandle InSceneHandle, FPBDRigidsSolver* InSolver);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void Shutdown();
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void RegisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void UnregisterBodyContainer(UE::Physics::IRigidBodyContainer* InContainer);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void CreateGeometryCollectionPT(const UE::Physics::FRigidBodyContainerHandle& InHandle, FGeometryCollectionPhysicsProxy* InProxy);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void DestroyGeometryCollectionPT(const UE::Physics::FRigidBodyContainerHandle& InHandle);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void CreateBodyPT(const UE::Physics::FRigidBodyHandle& InBodyHandle, FSingleParticlePhysicsProxy* InProxy);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void DestroyBodyPT(const UE::Physics::FRigidBodyHandle& InBodyHandle);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void RegisterBody(const UE::Physics::FRigidBodyHandle& InBodyHandle, UE::Physics::IRigidBody* InBody);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void UnRegisterBody(const UE::Physics::FRigidBodyHandle& InBodyHandle, const UE::Physics::IRigidBody* InBody);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void ActivateBody(const UE::Physics::FRigidBodyHandle& InBodyHandle);
		UE_INTERNAL CHAOSRIGIDPHYSICSASYNC_API void DeactivateBody(const UE::Physics::FRigidBodyHandle& InBodyHandle);

		// ShapeInstance
		UE_INTERNAL virtual FRigidShapeInstanceAsync* GetShape(const FRigidObjectId& InId) const override final;
		// ShapeInstance: Body
		UE_INTERNAL void AddShapePT(const FRigidBodyHandle BodyHandle, const FRigidShapeInstanceHandle ShapeInstanceHandle, const int32 ShapeIndex);
		UE_INTERNAL void RemoveShapePT(const FRigidBodyHandle BodyHandle, const FRigidShapeInstanceHandle ShapeInstanceHandle);
		UE_INTERNAL FRigidShapeInstanceAsync* GetShape(const FShapeInstanceView& InView, const int32 InIndex) const;

		// Joint Constraint
		UE_INTERNAL FJointConstraint6DOFAsyncPT* CreateJointConstraint6DOFPT(const FJointConstraint6DOFHandle& InHandle, Chaos::FJointConstraint* InJointConstraint);
		UE_INTERNAL void DestroyJointConstraintPT(const FJointConstraint6DOFHandle& InHandle);
		UE_INTERNAL virtual IJointConstraint* GetJointConstraint(const FRigidObjectId& InConstraintId) const override final;
		UE_INTERNAL void ActiveJointConstraint(const FJointConstraint6DOFHandle& InHandle);
		UE_INTERNAL void DeactivateJointConstraint(const FJointConstraint6DOFHandle& InHandle);

	private:
		void CheckGameContext() const;
		void CheckSimContext() const;

		void OnPreTick(FReal Dt);
		void OnPostTick(FReal Dt);

		UE::Physics::FRigidDebugName Name;
		UE::Physics::FRigidSceneHandle SceneHandle;
		FPBDRigidsSolver* Solver;
		TObjectPool<FRigidBodyAsyncPT> BodyPool;
		TMap<UE::Physics::FRigidObjectId, UE::Physics::IRigidBodyContainer*> BodyContainerRegistry;
		TMap<UE::Physics::FRigidObjectId, UE::Physics::IRigidBody*> BodyRegistry;
		TMap<UE::Physics::FRigidObjectId, UE::Physics::IJointConstraint*> JointConstraintRegistry;
		FShapeInstanceContainerAsyncPT ShapeInstanceContainer;
		TArray<UE::Physics::IRigidSceneModifier*> SceneModifiers;
		bool bIsStarted;
	};
}

#endif // UE_RIGIDPHYSICS_API_ENABLED
