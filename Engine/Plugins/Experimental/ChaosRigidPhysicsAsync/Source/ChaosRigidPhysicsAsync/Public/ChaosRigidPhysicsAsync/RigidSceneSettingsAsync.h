// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RigidPhysics/RigidFwd.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "ChaosSolverConfiguration.h"
#include "RigidPhysics/RigidSceneSettings.h"
#include "PhysicsCoreTypes.h"

namespace Chaos
{
	class FPBDRigidsSolver;
	class FSingleParticlePhysicsProxy;
}

namespace Chaos::Rigids::Async
{
	class UE_INTERNAL FRigidSceneSettingsAsync : public UE::Physics::IRigidSceneSettings
	{
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidSceneSettingsAsync);

		// The fixed timestep to use when async physics is enabled
		float AsyncDt = -1.0f;

		// The threading mode
		EChaosThreadingMode ThreadingMode = EChaosThreadingMode::SingleThread;

		// Solver settings
		FChaosSolverConfiguration SolverConfig;
	};
}

#endif // UE_RIGIDPHYSICS_API_ENABLED