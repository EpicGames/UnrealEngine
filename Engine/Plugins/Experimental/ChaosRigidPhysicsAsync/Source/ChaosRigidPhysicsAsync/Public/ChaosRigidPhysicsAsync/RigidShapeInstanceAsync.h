// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once 

#include "ChaosRigidPhysicsAsync/RigidFwdAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/CollisionFilterData.h"
#include "Math/Transform.h"
#include "RigidPhysics/RigidShapeInstance.h"

namespace Chaos::Rigids::Async
{
	class UE_INTERNAL FRigidShapeInstanceAsync : public UE::Physics::IRigidShapeInstance
	{
		using FShapeFilterData = Chaos::Filter::FShapeFilterData;
	public:
		UE_RIGIDPHYSICS_RIGIDTYPED_DECL(CHAOSRIGIDPHYSICSASYNC_API, FRigidShapeInstanceAsync);

		FRigidShapeInstanceAsync() = default;
		virtual ~FRigidShapeInstanceAsync() = default;

		UE_INTERNAL static Chaos::FPerShapeData* Create(const UE::Physics::FRigidShapeInstanceSetup& InSetup);

		UE_INTERNAL virtual UE::Physics::FRigidShapeInstanceHandle GetHandle() const override final;
		UE_INTERNAL virtual UE::Physics::FAnyGeometry GetGeometry() const override final;

		UE_INTERNAL virtual FTransform3f GetLocalTransform() const override final;

		UE_INTERNAL virtual bool GetQueryEnabled() const override final;
		UE_INTERNAL virtual void SetQueryEnabled(const bool bInEnabled) override final;

		UE_INTERNAL virtual bool GetSimEnabled() const override final;
		UE_INTERNAL virtual void SetSimEnabled(const bool bInEnabled) override final;

		UE_INTERNAL virtual bool GetIsProbe() const override final;
		UE_INTERNAL virtual void SetIsProbe(const bool bInIsProbe) override final;

		UE_INTERNAL virtual ECollisionTraceFlag GetCollisionTraceType() const override final;
		UE_INTERNAL virtual void SetCollisionTraceType(const ECollisionTraceFlag InTraceFlag) override final;

		UE_INTERNAL virtual FShapeFilterData GetShapeFilter() const override final;
		UE_INTERNAL virtual void SetShapeFilter(const FShapeFilterData& InShapeFilter) override final;

		UE_INTERNAL virtual Chaos::Filter::FInstanceData GetFilterInstanceData() const override final;
		UE_INTERNAL virtual void SetFilterInstanceData(const Chaos::Filter::FInstanceData& InInstanceData) override final;
		
		UE_INTERNAL virtual int32 GetNumMaterials() const override final;
		UE_INTERNAL virtual UE::Physics::FMaterialHandle GetMaterial(const int32 InIndex) const override final;
		UE_INTERNAL virtual void SetMaterials(TArray<UE::Physics::FMaterialHandle>&& InMaterials) override final;

		UE_INTERNAL virtual int32 GetNumMaterialMasks() const override final;
		UE_INTERNAL virtual UE::Physics::FMaterialMaskHandle GetMaterialMask(const int32 InIndex) const override final;
		UE_INTERNAL virtual void SetMaterialMasks(TArray<UE::Physics::FMaterialMaskHandle>&& InMaterialMasks) override final;

		UE_INTERNAL virtual int32 GetNumMaterialMaskMaps() const override final;
		UE_INTERNAL virtual uint32 GetMaterialMaskMap(const int32 InIndex) const override final;
		UE_INTERNAL virtual void SetMaterialMaskMaps(TArray<uint32>&& InMaterialMaskMaps) override final;

		UE_INTERNAL virtual int32 GetNumMaterialMaskMapMaterials() const override final;
		UE_INTERNAL virtual UE::Physics::FMaterialHandle GetMaterialMaskMapMaterial(const int32 InIndex) const override final;
		UE_INTERNAL virtual void SetMaterialMaskMapMaterials(TArray<UE::Physics::FMaterialHandle>&& InMaterialMaskMapMaterials) override final;

		UE_INTERNAL virtual void* GetUserData() const override final;
		UE_INTERNAL virtual void SetUserData(void* InUserData) override final;

		UE_INTERNAL Chaos::FPerShapeData* PerShapeData = nullptr;
		UE_INTERNAL UE::Physics::FRigidShapeInstanceHandle Handle;
		UE_INTERNAL bool bAttachedToBody = false;

	private:
		template <typename Lambda>
		static EImplicitObjectType GetLeafImplicit(FImplicitObject*& InOutImplicit, const Lambda& Func);
	};
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
