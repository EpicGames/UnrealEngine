// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosRigidPhysicsAsync/RigidFwdAsync.h"

#if UE_RIGIDPHYSICS_API_ENABLED

#include "Chaos/ObjectPool.h"
#include "ChaosRigidPhysicsAsync/RigidShapeInstanceAsync.h"
#include "RigidPhysics/RigidObjectRegistry.h"
#include "RigidPhysics/RigidSceneId.h"

namespace Chaos::Rigids::Async
{
	class UE_INTERNAL FShapeInstanceView
	{
	public:
		int32 Num() const;

	private:
		friend class FShapeInstanceContainerAsyncGT;
		friend class FShapeInstanceContainerAsyncPT;
		TArray<UE::Physics::FRigidObjectId> Ids;
	};

	class UE_INTERNAL FShapeInstanceContainerAsyncGT
	{
		using FRigidObjectId = UE::Physics::FRigidObjectId;
		using FRigidSceneId = UE::Physics::FRigidSceneId;
	public:
		FShapeInstanceContainerAsyncGT();

		FRigidShapeInstanceAsync* Allocate(const FRigidSceneId InSceneId);
		void Free(FRigidShapeInstanceAsync* InInstance);

		FRigidShapeInstanceAsync* Find(const FRigidObjectId& InId) const;

		int32 Add(FRigidShapeInstanceAsync& InInstance, FShapeInstanceView& InOutView);
		void Remove(FRigidShapeInstanceAsync& InInstance, FShapeInstanceView& InOutView);
		FRigidShapeInstanceAsync* Get(const FShapeInstanceView& InView, const int32 InIndex) const;
		void Free(FShapeInstanceView& InView);

	private:
		TObjectPool<FRigidShapeInstanceAsync> Pool;
		UE::Physics::TRigidObjectRegistry<FRigidShapeInstanceAsync> Registry;
	};

	class UE_INTERNAL FShapeInstanceContainerAsyncPT
	{
		using FRigidObjectId = UE::Physics::FRigidObjectId;
		using FRigidSceneId = UE::Physics::FRigidSceneId;
		using FRigidShapeInstanceHandle = UE::Physics::FRigidShapeInstanceHandle;

	public:
		FShapeInstanceContainerAsyncPT();

		FRigidShapeInstanceAsync* Allocate(const FRigidShapeInstanceHandle& InHandle);
		void Free(FRigidShapeInstanceAsync* InInstance);

		FRigidShapeInstanceAsync* Find(const FRigidObjectId& InId) const;

		void Add(FRigidShapeInstanceAsync& InInstance, const int32 InShapeIndex, FShapeInstanceView& InOutView);
		void Remove(FRigidShapeInstanceAsync& InInstance, FShapeInstanceView& InOutView);
		FRigidShapeInstanceAsync* Get(const FShapeInstanceView& InView, const int32 InIndex) const;
		void Free(FShapeInstanceView& InView);

	private:
		struct FRegistryItem
		{
			FRigidShapeInstanceAsync* Object = nullptr;
			int32 Epoch = 0;
		};

		TObjectPool<FRigidShapeInstanceAsync> Pool;
		TArray<FRegistryItem> Registry;
	};
} // namespace Chaos::Rigids::Async

#endif // UE_RIGIDPHYSICS_API_ENABLED
